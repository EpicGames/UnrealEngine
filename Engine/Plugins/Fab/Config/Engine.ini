[ConsoleVariables]
r.CEFGPUAcceleration=true
