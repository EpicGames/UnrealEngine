{
	"FileVersion": 3,
	"Version": 13,
	"VersionName": "0.0.13",
	"FriendlyName": "Fab",
	"Description": "Fab Plugin",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"Modules": [
		{
			"Name": "Fab",
			"Type": "Editor",
			"LoadingPhase": "PostDefault",
			"PlatformAllowList": [
				"Linux",
				"Win64",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "Interchange",
			"Enabled": true,
			"PlatformAllowList": [
				"Linux",
				"Win64",
				"Mac"
			]
		},
		{
			"Name": "EditorDataStorageFeatures",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "EOSShared",
			"Enabled": true,
			"PlatformAllowList": [
				"Linux",
				"Win64",
				"Mac"
			],
			"TargetAllowList": [
				"Editor"
			]
		}
	]
}