// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class Fab : ModuleRules
{
	public Fab(ReadOnlyTargetRules Target) : base(Target)
	{
		bRequiresPlatformSDK = true;
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		IWYUSupport = IWYUSupport.None;

		PublicIncludePaths.AddRange(
			new string[]
			{
				// ... add public include paths required here ...
				// Path.Combine(ModuleDirectory, "ThirdParty")
			}
		);

		PrivateIncludePaths.AddRange(
			new string[]
			{
				// ... add other private include paths required here ...
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"AssetTools",
				"BuildPatchServices",
				"ContentBrowser",
				"CoreUObject",
				"DesktopWidgets",
				"EditorScriptingUtilities",
				"EditorStyle",
				"EditorSubsystem",
				"Engine",
				"EOSSDK",
				"EOSShared",
				"FileUtilities",
				"Foliage",
				"GameProjectGeneration",
				"HTTP",
				"InputCore",
				"InterchangeCore",
				"InterchangeEngine",
				"InterchangePipelines",
				"Json",
				"JsonUtilities",
				"LevelEditor",
				"MainFrame",
				"PlacementMode",
				"Projects",
				"RenderCore",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"TypedElementFramework",
				"UMG",
				"UnrealEd",
				"WebBrowser",
				"InterchangeImport",
				"InterchangeNodes",
				"InterchangeFactoryNodes",
				"DeveloperSettings"
				// ... add private dependencies that you statically link with here ...
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
		);
	}
}
