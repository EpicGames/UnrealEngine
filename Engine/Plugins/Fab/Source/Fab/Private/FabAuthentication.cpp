// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabAuthentication.h"

#include "FabBrowser.h"
#include "FabLog.h"
#include "FabSettings.h"
#include "IEOSSDKManager.h"

#include "Async/Async.h"

#include "Misc/CommandLine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FabAuthentication)

namespace FabAuthentication
{
	void Init()
	{
		IEOSSDKManager* SDKManager = IEOSSDKManager::Get();
		if (SDKManager && SDKManager->IsInitialized())
		{
			const UFabSettings* FabSettings = GetDefault<UFabSettings>();
			FString ProductId;
			FString SandboxId;
			FString DeploymentId;
			FString ClientId;
			FString ClientSecret;
			FString EncryptionKey;
			const UEosConstants* const Constants = Cast<UEosConstants>(FSoftObjectPath("/Fab/Data/FabEos.FabEos").TryLoad());
			switch (FabSettings->Environment)
			{
			case EFabEnvironment::Prod:
				{
					ProductId = Constants->Prod.ProductId;
					SandboxId = Constants->Prod.SandboxId;
					DeploymentId = Constants->Prod.DeploymentId;
					ClientId = Constants->Prod.ClientCredentialsId;
					ClientSecret = Constants->Prod.ClientCredentialsSecret;
					EncryptionKey = Constants->Prod.EncryptionKey;
					break;
				}
			case EFabEnvironment::Gamedev:
				{
					ProductId = Constants->GameDev.ProductId;
					SandboxId = Constants->GameDev.SandboxId;
					DeploymentId = Constants->GameDev.DeploymentId;
					ClientId = Constants->GameDev.ClientCredentialsId;
					ClientSecret = Constants->GameDev.ClientCredentialsSecret;
					EncryptionKey = Constants->GameDev.EncryptionKey;
					break;
				}
			default:
				{
					ProductId = Constants->Prod.ProductId;
					SandboxId = Constants->Prod.SandboxId;
					DeploymentId = Constants->Prod.DeploymentId;
					ClientId = Constants->Prod.ClientCredentialsId;
					ClientSecret = Constants->Prod.ClientCredentialsSecret;
					EncryptionKey = Constants->Prod.EncryptionKey;
				}
			}

			const FTCHARToUTF8 Utf8ProductId(*ProductId);
			const FTCHARToUTF8 Utf8SandboxId(*SandboxId);
			const FTCHARToUTF8 Utf8ClientId(*ClientId);
			const FTCHARToUTF8 Utf8ClientSecret(*ClientSecret);
			const FTCHARToUTF8 Utf8EncryptionKey(*EncryptionKey);
			const FTCHARToUTF8 Utf8DeploymentId(*DeploymentId);
			//const FTCHARToUTF8 Utf8CacheDirectory(*CacheDirectory);

			EOS_Platform_Options PlatformOptions = {};
			PlatformOptions.ApiVersion = EOS_PLATFORM_OPTIONS_API_LATEST;

			PlatformOptions.ClientCredentials.ClientId = Utf8ClientId.Get();
			PlatformOptions.ClientCredentials.ClientSecret = Utf8ClientSecret.Get();
			PlatformOptions.ProductId = Utf8ProductId.Get();
			PlatformOptions.DeploymentId = Utf8DeploymentId.Get();
			PlatformOptions.SandboxId = Utf8SandboxId.Get();
			PlatformOptions.EncryptionKey = Utf8EncryptionKey.Get();
			// PlatformOptions.CacheDirectory = Utf8CacheDirectory.Get();
			PlatformOptions.bIsServer = EOS_FALSE;
			PlatformOptions.Flags = 0;
			PlatformOptions.Flags |= EOS_PF_DISABLE_OVERLAY;
			PlatformOptions.Reserved = nullptr;
			PlatformOptions.TickBudgetInMilliseconds = 0;
			PlatformOptions.IntegratedPlatformOptionsContainerHandle = nullptr;

			if (FabSettings->Environment == EFabEnvironment::Gamedev)
			{
				static struct FReservedOptions
				{
					int32_t ApiVersion;
					const char* BackendEnvironment;
				}
					ReservedOptions = {1, "GameDev"};
				PlatformOptions.Reserved = &ReservedOptions;
			}

			PlatformHandle = SDKManager->CreatePlatform(PlatformOptions);
		}
		else
		{
			FAB_LOG_ERROR("EOS is not initialized.");
		}
	}

	void Shutdown()
	{
		PlatformHandle.Reset();
	}

	void EOS_CALL ExchangeCodeLoginCompleteCallbackFn(const EOS_Auth_LoginCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			FAB_LOG("User logged in");
			const int32_t AccountsCount = EOS_Auth_GetLoggedInAccountsCount(AuthHandle);
			for (int32_t AccountIdx = 0; AccountIdx < AccountsCount; ++AccountIdx)
			{
				const EOS_EpicAccountId AccountId = EOS_Auth_GetLoggedInAccountByIndex(AuthHandle, AccountIdx);
				EOS_ELoginStatus LoginStatus = EOS_Auth_GetLoginStatus(AuthHandle, Data->LocalUserId);
				EpicAccountId = AccountId;
			}

			LoggedIn();
			return;
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_PinGrantCode)
		{
			FAB_LOG_ERROR("Login pin grant code");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_MFARequired)
		{
			FAB_LOG_ERROR("Login MFA required");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_InvalidUser)
		{
			FAB_LOG_ERROR("Invalid user");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_AccountFeatureRestricted)
		{
			FAB_LOG_ERROR("Login failed, account is restricted");
		}
		else if (EOS_EResult_IsOperationComplete(Data->ResultCode))
		{
			const FString Code = EOS_EResult_ToString(Data->ResultCode);
			FAB_LOG_ERROR("Login failed - error code: %s", *Code);
		}
		else
		{
			const FString Code = EOS_EResult_ToString(Data->ResultCode);
			FAB_LOG_ERROR("Login failed - error code: %s", *Code);
		}
	}

	void EOS_CALL PersistLoginCompleteCallbackFn(const EOS_Auth_LoginCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			FAB_LOG("User logged in");
			const int32_t AccountsCount = EOS_Auth_GetLoggedInAccountsCount(AuthHandle);
			for (int32_t AccountIdx = 0; AccountIdx < AccountsCount; ++AccountIdx)
			{
				const EOS_EpicAccountId AccountId = EOS_Auth_GetLoggedInAccountByIndex(AuthHandle, AccountIdx);
				EOS_ELoginStatus LoginStatus = EOS_Auth_GetLoginStatus(AuthHandle, Data->LocalUserId);
				EpicAccountId = AccountId;
			}

			LoggedIn();
			return;
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_PinGrantCode)
		{
			FAB_LOG_ERROR("Login pin grant code");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_MFARequired)
		{
			FAB_LOG_ERROR("Login MFA required");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_InvalidUser)
		{
			FAB_LOG_ERROR("Invalid user");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_AccountFeatureRestricted)
		{
			FAB_LOG_ERROR("Login failed, account is restricted");
		}
		else if (EOS_EResult_IsOperationComplete(Data->ResultCode))
		{
			const FString Code = EOS_EResult_ToString(Data->ResultCode);
			FAB_LOG_ERROR("Login failed - error code: %s", *Code);
		}
		else
		{
			const FString Code = EOS_EResult_ToString(Data->ResultCode);
			FAB_LOG_ERROR("Login failed - error code: %s", *Code);
		}

		// Fallback login using exchange code
		// Sending empty code reads from commandline
		LoginUsingExchangeCode("");
	}

	void EOS_CALL AccountPortalLoginCompleteCallbackFn(const EOS_Auth_LoginCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			FAB_LOG("User logged in");
			const int32_t AccountsCount = EOS_Auth_GetLoggedInAccountsCount(AuthHandle);
			for (int32_t AccountIdx = 0; AccountIdx < AccountsCount; ++AccountIdx)
			{
				const EOS_EpicAccountId AccountId = EOS_Auth_GetLoggedInAccountByIndex(AuthHandle, AccountIdx);
				EOS_ELoginStatus LoginStatus = EOS_Auth_GetLoginStatus(AuthHandle, Data->LocalUserId);
				EpicAccountId = AccountId;
			}

			LoggedIn();
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_PinGrantCode)
		{
			FAB_LOG_ERROR("Login pin grant code");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_MFARequired)
		{
			FAB_LOG_ERROR("Login MFA required");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_InvalidUser)
		{
			FAB_LOG_ERROR("Invalid user");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_Auth_AccountFeatureRestricted)
		{
			FAB_LOG_ERROR("Login failed, account is restricted");
		}
		else if (EOS_EResult_IsOperationComplete(Data->ResultCode))
		{
			const FString Code = EOS_EResult_ToString(Data->ResultCode);
			FAB_LOG_ERROR("Login failed - error code: %s", *Code);
		}
		else
		{
			const FString Code = EOS_EResult_ToString(Data->ResultCode);
			FAB_LOG_ERROR("Login failed - error code: %s", *Code);
		}
	}

	bool LoginUsingExchangeCode(FString ExchangeCode)
	{
		FAB_LOG("Logging in using exchange code");

		if (!PlatformHandle)
		{
			FAB_LOG_ERROR("Invalid EOS platform handle.");
			return false;
		}

		if (ExchangeCode.IsEmpty()) // Read exchange code from commandline if it is not passed in
		{
			FAB_LOG("Reading exchange code from commandline");
			FString AuthType;
			if (FParse::Value(FCommandLine::Get(), TEXT("AUTH_TYPE="), AuthType) && AuthType == TEXT("exchangecode"))
			{
				FParse::Value(FCommandLine::Get(), TEXT("AUTH_PASSWORD="), ExchangeCode);
			}
		}

		AuthHandle = EOS_Platform_GetAuthInterface(*PlatformHandle);

		EOS_Auth_Credentials Credentials = {0};
		Credentials.ApiVersion = EOS_AUTH_CREDENTIALS_API_LATEST;

		Credentials.Type = EOS_ELoginCredentialType::EOS_LCT_ExchangeCode;
		Credentials.Id = "";

		const FTCHARToUTF8 UTF8Token(*ExchangeCode);
		Credentials.Token = UTF8Token.Get();

		EOS_Auth_LoginOptions LoginOptions = {0};
		LoginOptions.ApiVersion = EOS_AUTH_LOGIN_API_LATEST;
		LoginOptions.Credentials = &Credentials;

		EOS_Auth_Login(AuthHandle, &LoginOptions, nullptr, ExchangeCodeLoginCompleteCallbackFn);

		return true;
	}

	bool LoginUsingPersist()
	{
		FAB_LOG("Logging in using persist");

		if (!PlatformHandle)
		{
			FAB_LOG_ERROR("Invalid EOS platform handle.");
			return false;
		}

		AuthHandle = EOS_Platform_GetAuthInterface(*PlatformHandle);

		EOS_Auth_Credentials Credentials = {0};
		Credentials.ApiVersion = EOS_AUTH_CREDENTIALS_API_LATEST;

		Credentials.Type = EOS_ELoginCredentialType::EOS_LCT_PersistentAuth;

		EOS_Auth_LoginOptions LoginOptions = {0};
		LoginOptions.ApiVersion = EOS_AUTH_LOGIN_API_LATEST;
		LoginOptions.Credentials = &Credentials;

		EOS_Auth_Login(AuthHandle, &LoginOptions, nullptr, PersistLoginCompleteCallbackFn);

		return true;
	}

	bool LoginUsingAccountPortal()
	{
		FAB_LOG("Logging in using account portal");

		if (!PlatformHandle)
		{
			FAB_LOG_ERROR("Invalid EOS platform handle.");
			return false;
		}

		AuthHandle = EOS_Platform_GetAuthInterface(*PlatformHandle);

		EOS_Auth_Credentials Credentials = {0};
		Credentials.ApiVersion = EOS_AUTH_CREDENTIALS_API_LATEST;

		Credentials.Type = EOS_ELoginCredentialType::EOS_LCT_AccountPortal;

		EOS_Auth_LoginOptions LoginOptions = {0};
		LoginOptions.ApiVersion = EOS_AUTH_LOGIN_API_LATEST;
		LoginOptions.Credentials = &Credentials;

		EOS_Auth_Login(AuthHandle, &LoginOptions, nullptr, AccountPortalLoginCompleteCallbackFn);

		return true;
	}

	void EOS_CALL DeletePersistentAuthCompleteCallbackFn(const EOS_Auth_DeletePersistentAuthCallbackInfo* Data)
	{
		if (Data->ResultCode == EOS_EResult::EOS_Success)
		{
			FAB_LOG("Persistent auth deleted");
		}
		else if (Data->ResultCode == EOS_EResult::EOS_NotFound)
		{
			FAB_LOG("Persistent auth not found - unable to delete");
		}
		else
		{
			FAB_LOG_ERROR("Unable to delete persistent auth");
		}
	}

	void PrintAuthToken(const EOS_Auth_Token* InAuthToken)
	{
		FAB_LOG("User client id: %s", *FString(InAuthToken->ClientId));
		// FAB_LOG("User access token: %s", *FString(InAuthToken->AccessToken));
		// FAB_LOG("User refresh token: %s", *FString(InAuthToken->RefreshToken));
	}

	void LoggedIn()
	{
		EOS_Auth_Token* UserAuthToken = nullptr;

		EOS_Auth_CopyUserAuthTokenOptions CopyTokenOptions = {0};
		CopyTokenOptions.ApiVersion = EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST;

		if (EOS_Auth_CopyUserAuthToken(AuthHandle, &CopyTokenOptions, EpicAccountId, &UserAuthToken) == EOS_EResult::EOS_Success)
		{
			const FString AccessToken = FString(UserAuthToken->AccessToken);
			PrintAuthToken(UserAuthToken);
			EOS_Auth_Token_Release(UserAuthToken);
			FFabBrowser::LoggedIn(AccessToken);
		}
		else
		{
			FAB_LOG_ERROR("User auth token is invalid");
		}
	}

	void EOS_CALL OnLogoutCallbackFn(const EOS_Auth_LogoutCallbackInfo* Data) 
	{
		if(Data->ResultCode != EOS_EResult::EOS_Success)
		{
			FAB_LOG_ERROR("EOS_Auth_Logout failed");
		}
		else
		{
			EOS_Auth_DeletePersistentAuthOptions Options = {};
			Options.ApiVersion = EOS_AUTH_DELETEPERSISTENTAUTH_API_LATEST;
			
			EOS_Auth_DeletePersistentAuth(AuthHandle, &Options, nullptr, DeletePersistentAuthCompleteCallbackFn);
		}
	}

	void DeletePersistentAuth()
	{
		FAB_LOG("Delete persist auth");

		EOS_Auth_LogoutOptions LogoutOptions = {};
		
		LogoutOptions.ApiVersion = EOS_AUTH_LOGOUT_API_LATEST;
		LogoutOptions.LocalUserId = EpicAccountId;
		
		EOS_Auth_Logout(AuthHandle, &LogoutOptions, nullptr, OnLogoutCallbackFn);
	}

	FString GetAuthToken()
	{
		EOS_Auth_Token* UserAuthToken = nullptr;

		EOS_Auth_CopyUserAuthTokenOptions CopyTokenOptions = {0};
		CopyTokenOptions.ApiVersion = EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST;

		if (EOS_Auth_CopyUserAuthToken(AuthHandle, &CopyTokenOptions, EpicAccountId, &UserAuthToken) == EOS_EResult::EOS_Success)
		{
			const FString AccessToken = FString(UserAuthToken->AccessToken);
			EOS_Auth_Token_Release(UserAuthToken);
			return AccessToken;
		}
		else
		{
			FAB_LOG_ERROR("User auth token is invalid - unable to get auth token");
			return "";
		}
	}
	
	FString GetRefreshToken()
	{
		EOS_Auth_Token* UserAuthToken = nullptr;

		EOS_Auth_CopyUserAuthTokenOptions CopyTokenOptions = {0};
		CopyTokenOptions.ApiVersion = EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST;

		if (EOS_Auth_CopyUserAuthToken(AuthHandle, &CopyTokenOptions, EpicAccountId, &UserAuthToken) == EOS_EResult::EOS_Success)
		{
			const FString RefreshToken = FString(UserAuthToken->RefreshToken);
			EOS_Auth_Token_Release(UserAuthToken);
			return RefreshToken;
		}
		else
		{
			FAB_LOG_ERROR("User auth token is invalid - unable to get refresh token");
			return "";
		}
	}
}
