// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "eos_auth.h"
#include "eos_common.h"
#include "eos_sdk.h"

#include "Engine/DataAsset.h"

#include "FabAuthentication.generated.h"

USTRUCT()
struct FEosConstantsGameDev
{
	GENERATED_BODY()

	/** The product id for the running application, found on the dev portal */
	UPROPERTY()
	FString ProductId;

	/** The sandbox id for the running application, found on the dev portal */
	UPROPERTY()
	FString SandboxId;

	/** The deployment id for the running application, found on the dev portal */
	UPROPERTY()
	FString DeploymentId;

	/** Client id of the service permissions entry, found on the dev portal */
	UPROPERTY()
	FString ClientCredentialsId;

	/** Client secret for accessing the set of permissions, found on the dev portal */
	UPROPERTY()
	FString ClientCredentialsSecret;

	/** Game name */
	UPROPERTY()
	FString GameName;

	/** Encryption key. */
	UPROPERTY()
	FString EncryptionKey;

	/** Product Version. */
	UPROPERTY()
	FString ProductVersion;
};

USTRUCT()
struct FEosConstantsProd
{
	GENERATED_BODY()

	/** The product id for the running application, found on the dev portal */
	UPROPERTY()
	FString ProductId;

	/** The sandbox id for the running application, found on the dev portal */
	UPROPERTY()
	FString SandboxId;

	/** The deployment id for the running application, found on the dev portal */
	UPROPERTY()
	FString DeploymentId;

	/** Client id of the service permissions entry, found on the dev portal */
	UPROPERTY()
	FString ClientCredentialsId;

	/** Client secret for accessing the set of permissions, found on the dev portal */
	UPROPERTY()
	FString ClientCredentialsSecret;

	/** Game name */
	UPROPERTY()
	FString GameName;

	/** Encryption key. */
	UPROPERTY()
	FString EncryptionKey;

	/** Product Version. */
	UPROPERTY()
	FString ProductVersion;
};

UCLASS()
class UEosConstants : public UDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FEosConstantsGameDev GameDev;

	UPROPERTY()
	FEosConstantsProd Prod;
};

using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle>;

/**
* Manages all user authentication
*/
namespace FabAuthentication
{
	inline IEOSPlatformHandlePtr PlatformHandle;
	inline EOS_HAuth AuthHandle;
	inline EOS_EpicAccountId EpicAccountId;

	void Init();
	void Shutdown();

	bool LoginUsingExchangeCode(FString ExchangeCode);
	bool LoginUsingRefreshToken(FString RefreshToken);
	bool LoginUsingAccountPortal();
	bool LoginUsingPersist();

	FString GetAuthToken();
	FString GetRefreshToken();

	/**
	* Deletes any locally stored persistent auth credentials for the currently logged in user of the local device.
	*/
	void DeletePersistentAuth();

	/**
	* Utility for printing auth token info
	*/
	void PrintAuthToken(const EOS_Auth_Token* InAuthToken);

	/** Called when successfully logged in */
	void LoggedIn();

	/**
	* Callback that is fired when the login operation completes, either successfully or in error
	*/
	static void EOS_CALL ExchangeCodeLoginCompleteCallbackFn(const EOS_Auth_LoginCallbackInfo* Data);

	/**
	* Callback that is fired when the login operation completes, either successfully or in error
	*/
	static void EOS_CALL AccountPortalLoginCompleteCallbackFn(const EOS_Auth_LoginCallbackInfo* Data);

	/**
	* Callback that is fired when the login operation completes, either successfully or in error
	*/
	static void EOS_CALL PersistLoginCompleteCallbackFn(const EOS_Auth_LoginCallbackInfo* Data);

	/**
	* Callback that is fired when the delete persistent auth operation completes, either successfully or in error
	*/
	static void EOS_CALL DeletePersistentAuthCompleteCallbackFn(const EOS_Auth_DeletePersistentAuthCallbackInfo* Data);

	static void EOS_CALL OnLogoutCallbackFn(const EOS_Auth_LogoutCallbackInfo* Data);
}
