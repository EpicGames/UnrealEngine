// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabBrowser.h"

#include "ContentBrowserModule.h"
#include "FabAuthentication.h"
#include "FabLog.h"
#include "FabSettings.h"
#include "FabSettingsWindow.h"
#include "IWebBrowserPopupFeatures.h"
#include "IWebBrowserWindow.h"
#include "JsonObjectConverter.h"
#include "LevelEditor.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "WebBrowserModule.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

#include "Interfaces/IMainFrameModule.h"
#include "Interfaces/IPluginManager.h"

#include "Math/Vector2D.h"

#include "Runtime/Launch/Resources/Version.h"

#if PLATFORM_WINDOWS && (ENGINE_MAJOR_VERSION > 5 || (ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION >= 7))
#include "Windows/WindowsApplication.h"
#endif

#include "Misc/FileHelper.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#include "UObject/GCObject.h"

#include "Utilities/FabLocalAssets.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FabBrowser)

#define LOCTEXT_NAMESPACE "Fab"

TUniquePtr<FSlateStyleSet> FFabBrowser::SlateStyleSet         = nullptr;
TObjectPtr<const UFabSettings> FFabBrowser::FabPluginSettings = nullptr;

int32 FFabBrowser::FabTabCounter = 0;
FName FFabBrowser::ActiveFabTabId = NAME_None;
TMap<FName, FFabBrowserTabContext> FFabBrowser::TabsById = {};
TSharedPtr<FTabManager> FFabBrowser::TabManager;
bool FFabBrowser::bIsShuttingDown = false;

const FText FFabBrowser::FabLabel           = LOCTEXT("Fab.Label", "Fab");
const FText FFabBrowser::FabDefaultTabTitle = LOCTEXT("Fab.DefaultTabTitle", "Fab Plugin");
const FText FFabBrowser::FabTooltip         = LOCTEXT("Fab.Tooltip", "Get content from Fab");
const FName FFabBrowser::FabMenuIconName    = TEXT("Fab.MenuIcon");
const FName FFabBrowser::FabAssetIconName   = TEXT("Fab.AssetIcon");
const FName FFabBrowser::FabToolbarIconName = TEXT("Fab.ToolbarIcon");

static const FName HostTabId(TEXT("FabBrowser.Host"));

// UE 5.7 upgraded CEF, which changed how native-rendered browser windows track
// their viewport when the parent editor window is moved or resized. This message
// handler intercepts WM_EXITSIZEMOVE to refresh the viewport and reparent tabs.
#if PLATFORM_WINDOWS && (ENGINE_MAJOR_VERSION > 5 || (ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION >= 7))
class FFabWindowMoveMessageHandler final : public IWindowsMessageHandler
{
public:
	~FFabWindowMoveMessageHandler() = default;

	TFunction<void()> OnMoveEnded;

	virtual bool ProcessMessage(HWND, uint32 msg, WPARAM, LPARAM, int32&) override
	{
		if (msg == WM_EXITSIZEMOVE && OnMoveEnded)
		{
			OnMoveEnded();
			return false;
		}
		return false;
	}
};

static TSharedPtr<FFabWindowMoveMessageHandler> WindowMoveMessageHandler;
#endif

void FFabBrowser::Init()
{
	bIsShuttingDown = false;
	RegisterSlateStyle();
	SetupEntryPoints();
	ExtendContextMenuInContentBrowser();
}

void FFabBrowser::ExtendContextMenuInContentBrowser()
{
	FContentBrowserModule& ContentBrowserModule                       = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray<FContentBrowserMenuExtender_SelectedAssets>& MenuExtenders = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	FAssetViewExtraStateGenerator StateGenerator(
		FOnGenerateAssetViewExtraStateIndicators::CreateStatic(&FFabBrowser::OnFabAssetIconGenerate),
		FOnGenerateAssetViewExtraStateIndicators()
	);
	ContentBrowserModule.AddAssetViewExtraStateGenerator(StateGenerator);
	MenuExtenders.Add(FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FFabBrowser::OnExtendContentBrowserAssetSelectionMenu));
}

void FFabBrowser::RegisterSlateStyle()
{
	SlateStyleSet = MakeUnique<FSlateStyleSet>(TEXT("FabStyle"));
	SlateStyleSet->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("Fab"))->GetBaseDir() / TEXT("Resources"));

	const FString IconPath          = SlateStyleSet->RootToContentDir(TEXT("FabLogo.svg"));
	const FString AlternateIconPath = SlateStyleSet->RootToContentDir(TEXT("FabLogoAlternate.svg"));
	SlateStyleSet->Set(FabMenuIconName, new FSlateVectorImageBrush(IconPath, CoreStyleConstants::Icon16x16));
	SlateStyleSet->Set(FabAssetIconName, new FSlateVectorImageBrush(AlternateIconPath, CoreStyleConstants::Icon20x20));
	SlateStyleSet->Set(FabToolbarIconName, new FSlateVectorImageBrush(IconPath, CoreStyleConstants::Icon20x20));

	FSlateStyleRegistry::RegisterSlateStyle(*SlateStyleSet);

	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

void FFabBrowser::SetupEntryPoints()
{
	const FUIAction InvokeTabAction = FUIAction(
		FExecuteAction::CreateLambda(
			[]()
			{
				CreateNewFabTab();
			}
		),
		FCanExecuteAction()
	);

	{
		FToolMenuSection& SaveSection = UToolMenus::Get()->ExtendMenu("ContentBrowser.Toolbar")->FindOrAddSection("Save");
		FToolMenuEntry& ToolMenuEntry = SaveSection.AddEntry(
			FToolMenuEntry::InitToolBarButton(
				"OpenFabWindow",
				InvokeTabAction,
				FabLabel,
				FabTooltip,
				FSlateIcon(SlateStyleSet->GetStyleSetName(), FabToolbarIconName),
				EUserInterfaceActionType::Button
			)
		);
		ToolMenuEntry.InsertPosition.Position = EToolMenuInsertType::Last;
		ToolMenuEntry.StyleNameOverride = "ContentBrowser.ToolBar.Buttons";
	}

	FToolMenuEntry FabMenuEntry = FToolMenuEntry::InitMenuEntry(
		"OpenFabTab",
		FabLabel,
		FabTooltip,
		FSlateIcon(SlateStyleSet->GetStyleSetName(), FabMenuIconName),
		InvokeTabAction
	);

	{
		UToolMenu* WindowMenu = UToolMenus::Get()->ExtendMenu("MainFrame.MainMenu.Window");
		FToolMenuSection& ContentSection = WindowMenu->FindOrAddSection("GetContent", NSLOCTEXT("MainAppMenu", "GetContentHeader", "Get Content"));
		FToolMenuEntry& FabEntry = ContentSection.AddEntry(FabMenuEntry);
		FabEntry.InsertPosition.Position = EToolMenuInsertType::First;
	}

	// Add a Fab entry to the Content Browser's Add popup menu
	UToolMenus::Get()->ExtendMenu(TEXT("ContentBrowser.AddNewContextMenu"))->AddSection(TEXT("ContentBrowserGetContent"), LOCTEXT("GetContentText", "Get Content")).AddEntry(
		FToolMenuEntry::InitMenuEntry(TEXT("OpenFabWindow"), FabLabel, FabTooltip, FSlateIcon(SlateStyleSet->GetStyleSetName(), FabMenuIconName), InvokeTabAction)
	);

	{
		FToolMenuEntry& FabEntry = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.AddQuickMenu")->FindOrAddSection("Content").AddEntry(FabMenuEntry);
		FabEntry.InsertPosition.Name = "ImportContent";
		FabEntry.InsertPosition.Position = EToolMenuInsertType::After;
	}
}

// Extend the context menu to view listings in Fab
TSharedRef<FExtender> FFabBrowser::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	if (SelectedAssets.Num() != 1)
	{
		return Extender;
	}
	const FAssetData AssetData = SelectedAssets[0];
	const FString ObjectPath   = AssetData.GetObjectPathString();
	FString FabListingId;
	UFabLocalAssets::GetListingID(ObjectPath, FabListingId);

	if (FabListingId.IsEmpty())
	{
		return Extender;
	}

	Extender->AddMenuExtension(
		"CommonAssetActions",
		EExtensionHook::After,
		nullptr,
		FMenuExtensionDelegate::CreateLambda(
			[FabListingId](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuEntry(
					LOCTEXT("Fab.ViewInFab", "View in Fab"),
					LOCTEXT("Fab.ViewInFabTooltip", "View the asset in Fab plugin"),
					FSlateIcon(SlateStyleSet->GetStyleSetName(), FabMenuIconName),
					FUIAction(
						FExecuteAction::CreateLambda(
							[FabListingId]()
							{
								FFabBrowser::OpenURL(GetUrl() / "listings" / FabListingId);
							}
						)
					)
				);
			}
		)
	);
	return Extender;
}

TSharedRef<SWidget> FFabBrowser::OnFabAssetIconGenerate(const FAssetData& AssetData)
{
	const FSlateBrush* FabImage = nullptr;

	const FString ObjectPath = AssetData.GetObjectPathString();
	FString FabListingId;
	UFabLocalAssets::GetListingID(ObjectPath, FabListingId);

	if (!FabListingId.IsEmpty())
	{
		FabImage = SlateStyleSet->GetBrush(FabAssetIconName);
	}

	return SNew(SBox)
		.Padding(4.0f, 4.0f, 0.0f, 0.0f)
		.IsEnabled(FabImage != nullptr)
		[
			SNew(SImage)
			.Image(FabImage)
			.ToolTipText(LOCTEXT("Fab.ImportedFromFab", "Imported from FAB"))
		];
}

// Helpers

FName FFabBrowser::MakeNextFabTabId()
{
	++FabTabCounter;
	const FString TabName = FString::Printf(TEXT("Fab%d"), FabTabCounter);
	return FName(*TabName);
}

bool FFabBrowser::EnsureWebBrowserAvailable()
{
	// // Call Get() first — it loads the module if needed. Checking IsAvailable()
	// // before Get() would fail when the module simply hasn't been loaded yet.
	const IWebBrowserModule& WebBrowserModule = IWebBrowserModule::Get();
	if (!IWebBrowserModule::IsAvailable() || !WebBrowserModule.IsWebModuleAvailable())
	{
		FAB_LOG_ERROR("WebBrowserModule isn't available! Can't display Fab tab contents.");
		return false;
	}

	return true;
}

static TSharedRef<SWidget> MakeErrorContent()
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(0, 32, 0, 0))
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
				.Text(LOCTEXT("Fab.BrowserError", "Unable to display the Fab Browser"))
				.Justification(ETextJustify::Center)
		];
}

static void SpawnErrorTab()
{
	const FName ErrorTabId(TEXT("FabError"));
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ErrorTabId);
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		ErrorTabId,
		FOnSpawnTab::CreateLambda([](const FSpawnTabArgs&)
		{
			return SNew(SDockTab)
				.TabRole(ETabRole::NomadTab)
				[
					MakeErrorContent()
				];
		})
	)
	.SetDisplayName(LOCTEXT("Fab.Label", "Fab"))
	.SetMenuType(ETabSpawnerMenuType::Hidden);

	FGlobalTabmanager::Get()->TryInvokeTab(ErrorTabId);
}

FString FFabBrowser::BuildIndexFileUrl()
{
	const FString PluginPath = IPluginManager::Get().FindPlugin(TEXT("Fab"))->GetBaseDir();
	const FString IndexPath  = FPaths::ConvertRelativePathToFull(
		FPaths::Combine(PluginPath, TEXT("ThirdParty"), TEXT("index.html"))
	);

	// "file:///" + absolute path
	return FPaths::Combine(TEXT("file:///"), IndexPath);
}

// Tab Creation

void FFabBrowser::CreateNewFabTab(const FString& Url)
{
	FabAuthentication::LoginUsingPersist();

	if (!FabPluginSettings)
	{
		FabPluginSettings = GetDefault<UFabSettings>();
	}

	if (!EnsureWebBrowserAvailable())
	{
		SpawnErrorTab();
		return;
	}

#if PLATFORM_MAC
	// Mac: single OSR instance. If a tab is already open, focus it
	// (and navigate to Url if non-empty) instead of creating another.
	for (const auto& Pair : TabsById)
	{
		if (!Pair.Value.bIsClosing)
		{
			if (TSharedPtr<SDockTab> Tab = Pair.Value.DockTab.Pin())
			{
				if (!Url.IsEmpty() && Pair.Value.WebBrowserWindow.IsValid())
				{
					Pair.Value.WebBrowserWindow->LoadURL(Url);
				}
				Tab->FlashTab();
				Tab->ActivateInParent(ETabActivationCause::SetDirectly);
				ActiveFabTabId = Pair.Key;
				return;
			}
		}
	}
	CreateOsrFabTab(Url);
#else
	// On Windows, respect the user's rendering mode preference
	if (FabPluginSettings->ExperienceMode == EFabExperienceMode::Performance)
	{
		CreateNativeFabTab(Url);
	}
	else
	{
		CreateOsrFabTab(Url);
	}
#endif
}

void FFabBrowser::CreateNativeFabTab(const FString& Url)
{
	// Create window if needed
	if (!TabManager.IsValid())
	{
		const TSharedRef<SDockTab> RootDockTab =
			SNew(SDockTab)
			.TabRole(MajorTab);

		TabManager = FGlobalTabmanager::Get()->NewTabManager(RootDockTab);

		TabManager->InsertNewDocumentTab(
			HostTabId,
			FTabManager::ESearchPreference::PreferLiveTab,
			RootDockTab
		);
	}

	const FName TabId = MakeNextFabTabId();

	// Create a new major tab
	const TSharedRef<SDockTab> DockTab =
		SNew(SDockTab)
		.TabRole(DocumentTab)
		.Label(FabDefaultTabTitle)
		.OnTabClosed_Lambda([TabId](TSharedRef<SDockTab> Tab) { HandleTabClosed(Tab, TabId); });

	// Pre-register the tab context so HandleTabClosed can find and mark it
	// as closing if the user closes the tab before the deferred attach fires.
	FFabBrowserTabContext& PreContext = TabsById.FindOrAdd(TabId);
	PreContext.TabId = TabId;
	PreContext.DockTab = DockTab;

	TabManager->InsertNewDocumentTab(
		HostTabId,
		FTabManager::ESearchPreference::PreferLiveTab,
		DockTab
	);

	// Defer browser attach to the next frame so the DockTab has a valid
	// parent window and correct bounds. Attaching synchronously before the
	// tab is fully docked causes the native CEF window to briefly fill the
	// entire screen.
	// Capture a weak pointer to break the reference cycle (the active timer
	// is owned by DockTab, so capturing a TSharedRef would prevent destruction).
	TWeakPtr<SDockTab> WeakDockTab = DockTab;
	DockTab->RegisterActiveTimer(0.0f,
		FWidgetActiveTimerDelegate::CreateLambda(
			[WeakDockTab, TabId, Url](double /*InCurrentTime*/, float /*InDeltaTime*/) -> EActiveTimerReturnType
			{
				TSharedPtr<SDockTab> PinnedTab = WeakDockTab.Pin();
				if (!PinnedTab.IsValid())
				{
					return EActiveTimerReturnType::Stop;
				}

				// If the tab was already closed between insertion and this callback, bail out.
				if (FFabBrowserTabContext* PreRegistered = TabsById.Find(TabId))
				{
					if (PreRegistered->bIsClosing)
					{
						return EActiveTimerReturnType::Stop;
					}
				}

				AttachBrowserToDockTab(PinnedTab.ToSharedRef(), /*bNative*/ true, TabId, Url);
				SetupTabWindowMoveListenerForNativeRendering(PinnedTab.ToSharedRef());
				return EActiveTimerReturnType::Stop;
			}
		)
	);
}

void FFabBrowser::CreateOsrFabTab(const FString& Url)
{
#if (ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION <= 6)
	// Limit the number of OSR tabs that can be opened
	// due to the problem of browser freezing on earlier
	// engine versions (5.3-5.6)

	// Count the current number of OSR tabs
	int32 Count = 0;
	for (const TPair<FName, FFabBrowserTabContext>& Tab : TabsById)
	{
		if (!Tab.Value.bIsNativelyRendered)
		{
			Count += 1;
		}
	}

	if (Count > 3)
	{
		CreateNativeFabTab(Url);
		return;
	}
#endif

	const FName TabId = MakeNextFabTabId();
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		TabId,
		FOnSpawnTab::CreateLambda([TabId, Url](const FSpawnTabArgs&)
		{
			const TSharedRef<SDockTab> DockTab =
				SNew(SDockTab)
				.TabRole(ETabRole::NomadTab)
				.Label(FabDefaultTabTitle)
				.OnTabClosed_Lambda([TabId](TSharedRef<SDockTab> Tab) { HandleTabClosed(Tab, TabId); })
				.OnTabActivated_Static(&FFabBrowser::HandleTabActivated, TabId);

			AttachBrowserToDockTab(DockTab, /*bNative*/ false, TabId, Url);
			return DockTab;
		})
	)
	.SetDisplayName(FabDefaultTabTitle)
	.SetMenuType(ETabSpawnerMenuType::Hidden);

	FGlobalTabmanager::Get()->TryInvokeTab(TabId);
}

void FFabBrowser::PreventWebBrowserFreezeInOsrMode(const FName& TabId, const FFabBrowserTabContext& Context)
{
	if (!Context.bIsNativelyRendered)
	{
		Context.WebBrowserInstance->SetOnMouseEnter(FNoReplyPointerEventHandler::CreateLambda([TabId](const FGeometry& Geometry, const FPointerEvent& PointerEvent)
		{
			// Disable all other web browser instances
			for (const TTuple<FName, FFabBrowserTabContext>& Element : TabsById)
			{
				// Don't disable the active tab
				if (ActiveFabTabId != Element.Key && !Element.Value.bIsNativelyRendered && Element.Value.WebBrowserWindow.IsValid())
				{
					Element.Value.WebBrowserWindow->SetIsDisabled(true);
				}
			}

			// Enable the one in focus
			if (FFabBrowserTabContext* const& FabBrowserTabContext = TabsById.Find(TabId))
			{
				if (FabBrowserTabContext->WebBrowserWindow.IsValid())
				{
					FabBrowserTabContext->WebBrowserWindow->SetIsDisabled(false);
				}
			}
		}));
	}
}

// Pop the JS bridge UObject off the GC root set and clear the weak pointer.
// Safe to call regardless of current liveness.
static void ReleaseJavascriptApi(FFabBrowserTabContext& Context)
{
	if (UFabBrowserApi* Api = Context.JavascriptApi.Get())
	{
		Api->RemoveFromRoot();
	}
	Context.JavascriptApi.Reset();
}

void FFabBrowser::AttachBrowserToDockTab(const TSharedRef<SDockTab>& DockTab, const bool bNativeRendering, const FName TabId, const FString& Url)
{
	FFabBrowserTabContext& Context = TabsById.FindOrAdd(TabId);
	Context.TabId = TabId;
	Context.DockTab = DockTab;

	// Create JS API — each tab should get exactly one bridge object.
	if (Context.JavascriptApi.IsValid())
	{
		ensureMsgf(false, TEXT("AttachBrowserToDockTab called with an existing JavascriptApi on TabId %s; cleaning up."), *TabId.ToString());
		ReleaseJavascriptApi(Context);
	}
	UFabBrowserApi* NewApi = NewObject<UFabBrowserApi>();
	Context.JavascriptApi = NewApi;
	NewApi->AddToRoot();

	// Browser settings
	FCreateBrowserWindowSettings Settings;
	Settings.InitialURL = Url.IsEmpty() ? BuildIndexFileUrl() : Url;
	// Override the user agent to avoid issues with the project name
	// If the project name has the word "agent" in it, it'll cause the captcha to fail
	Settings.UserAgentApplication = IWebBrowserModule::MakeUserAgentApplication(TEXT("Fab"));

#if PLATFORM_MAC
	Settings.bMobileJSReturnInDict = false;
#endif

	Context.bIsNativelyRendered = bNativeRendering;

	if (bNativeRendering)
	{
		if (DockTab->GetParentWindow().IsValid())
		{
			Settings.OSWindowHandle = DockTab->GetParentWindow()->GetNativeWindow()->GetOSWindowHandle();
		}

		SetupDockTabListenersForNativeRendering(DockTab, TabId);
	}

	IWebBrowserSingleton* Singleton = IWebBrowserModule::Get().GetSingleton();
	Singleton->SetDevToolsShortcutEnabled(true);
	Context.WebBrowserWindow = Singleton->CreateBrowserWindow(Settings);

	if (!Context.WebBrowserWindow.IsValid())
	{
		FAB_LOG_ERROR("WebBrowserWindow is invalid! Can't display Fab tab contents.");
		ReleaseJavascriptApi(Context);
		DockTab->SetContent(MakeErrorContent());
		return;
	}

	PreventBrowserKeyConflictWithEditor(Context);
	InstallUrlRestrictions(Context);
	InstallDebugPopup(Context);

	const bool bShowAddressBar = (FabPluginSettings->Environment == EFabEnvironment::CustomUrl);
	SAssignNew(Context.WebBrowserInstance, SWebBrowser, Context.WebBrowserWindow)
	.ShowAddressBar(bShowAddressBar)
	.ShowControls(bShowAddressBar)
	.OnTitleChanged_Lambda([TabId](const FText& Title)
	{
		if (FFabBrowserTabContext* const& FabBrowserTabContext = TabsById.Find(TabId))
		{
			if (const TSharedPtr<SDockTab>& Tab = FabBrowserTabContext->DockTab.Pin())
			{
				if (!Title.IsEmpty())
				{
					Tab->SetLabel(Title);
				}
			}
		}
	});

	PreventWebBrowserFreezeInOsrMode(TabId, Context);

	DockTab->SetContent(Context.WebBrowserInstance.ToSharedRef());

	Context.WebBrowserInstance->BindUObject(TEXT("fab"), Cast<UObject>(Context.JavascriptApi.Get()), true);
	Context.WebBrowserWindow->Reload();

	ActiveFabTabId = TabId;

#if PLATFORM_MAC && !WITH_CEF3
	FSlateApplication::Get().ToggleDisableLastDragOnDragEnter(true);
#endif
}

void FFabBrowser::SetupDockTabListenersForNativeRendering(const TSharedRef<SDockTab>& DockTab, const FName TabId)
{
	DockTab->SetOnTabDraggedOverDockArea(FSimpleDelegate::CreateLambda([TabId]
	{
		// Disable the web browser that is being dragged
		if (FFabBrowserTabContext* const& TabContext = TabsById.Find(TabId))
		{
			if (!TabContext->bIsClosing && TabContext->WebBrowserWindow.IsValid())
			{
				TabContext->WebBrowserWindow->SetIsDisabled(true);
			}
		}
	}));

	TWeakPtr<SDockTab> WeakRelocatedTab = DockTab;
	DockTab->SetOnTabRelocated(FSimpleDelegate::CreateLambda([WeakRelocatedTab]
	{
		if (WeakRelocatedTab.Pin().IsValid())
		{
			RefreshNativeTabViewPort();
			ReparentDockTabs();
		}
	}));

	DockTab->SetOnTabActivated(SDockTab::FOnTabActivatedCallback::CreateLambda([](TSharedRef<SDockTab>, ETabActivationCause)
	{
		ReparentDockTabs();
	}));
}

void FFabBrowser::SetupTabWindowMoveListenerForNativeRendering(const TSharedRef<SDockTab>& DockTab)
{
	const TSharedPtr<SWindow> ParentWindow = DockTab->GetParentWindow();
	if (!ParentWindow.IsValid())
	{
		return;
	}

#if PLATFORM_WINDOWS && (ENGINE_MAJOR_VERSION > 5 || (ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION >= 7))
	FWindowsApplication* WindowsApp = static_cast<FWindowsApplication*>(FSlateApplication::Get().GetPlatformApplication().Get());
	if (!WindowsApp)
	{
		return;
	}

	if (!WindowMoveMessageHandler.IsValid())
	{
		WindowMoveMessageHandler = MakeShared<FFabWindowMoveMessageHandler>();
		WindowMoveMessageHandler->OnMoveEnded = []()
		{
			RefreshNativeTabViewPort();
			ReparentDockTabs();
		};
		WindowsApp->AddMessageHandler(*WindowMoveMessageHandler);
	}
#endif
}

void FFabBrowser::ReparentDockTabs()
{
	for (const TTuple<FName, FFabBrowserTabContext>& Pair : TabsById)
	{
		const FFabBrowserTabContext& TabContext = Pair.Value;
		if (!TabContext.bIsNativelyRendered || !TabContext.WebBrowserInstance.IsValid() || !TabContext.WebBrowserWindow.IsValid())
		{
			continue;
		}

		TabContext.WebBrowserWindow->SetIsDisabled(false);

		const TSharedPtr<SDockTab>& Tab = TabContext.DockTab.Pin();
		if (Tab.IsValid() && Tab->GetParentWindow().IsValid())
		{
			TabContext.WebBrowserWindow->SetParentWindow(Tab->GetParentWindow());
		}
	}
}

void FFabBrowser::RefreshNativeTabViewPort()
{
	for (const TTuple<FName, FFabBrowserTabContext>& Pair : TabsById)
	{
		const FFabBrowserTabContext& TabContext = Pair.Value;
		if (!TabContext.bIsNativelyRendered || !TabContext.WebBrowserInstance.IsValid() || !TabContext.WebBrowserWindow.IsValid())
		{
			continue;
		}

		TabContext.WebBrowserWindow->SetIsDisabled(false);

		const TSharedPtr<SDockTab>& DockTab = TabContext.DockTab.Pin();
		if (!DockTab.IsValid())
		{
			continue;
		}

		if (const TSharedPtr<SWindow>& ParentWindow = DockTab->GetParentWindow(); ParentWindow.IsValid())
		{
			const FVector2D WindowSize = ParentWindow->GetViewportSize();
			const FGeometry& CachedGeometry = TabContext.WebBrowserInstance->GetCachedGeometry();
			const UE::Math::TVector2<double>& BrowserPosition = CachedGeometry.GetAbsolutePosition();

			TabContext.WebBrowserWindow->SetViewportSize(
				FIntPoint(FMath::RoundToInt(WindowSize.X), FMath::RoundToInt(WindowSize.Y)),
				FIntPoint(FMath::RoundToInt(BrowserPosition.X), FMath::RoundToInt(BrowserPosition.Y))
			);
		}
	}
}

void FFabBrowser::PreventBrowserKeyConflictWithEditor(const FFabBrowserTabContext& Context)
{
	Context.WebBrowserWindow->OnUnhandledKeyUp().BindLambda([](const FKeyEvent&) { return true; });
	Context.WebBrowserWindow->OnUnhandledKeyDown().BindLambda([](const FKeyEvent&) { return true; });
}

void FFabBrowser::InstallUrlRestrictions(const FFabBrowserTabContext& Context)
{
	if (!FabPluginSettings || FabPluginSettings->Environment == EFabEnvironment::CustomUrl)
	{
		return;
	}

	TWeakPtr<IWebBrowserWindow> WeakBrowserWindow = Context.WebBrowserWindow;

	Context.WebBrowserWindow->OnUrlChanged().AddLambda([WeakBrowserWindow](const FString& Url)
	{
		TSharedPtr<IWebBrowserWindow> BrowserWindow = WeakBrowserWindow.Pin();
		if (!BrowserWindow.IsValid())
		{
			return;
		}

		if (!FabPluginSettings || FabPluginSettings->Environment == EFabEnvironment::CustomUrl)
		{
			return;
		}

		InterceptNavigation(BrowserWindow, Url, FabPluginSettings->GetUrlFromEnvironment());
	});

	Context.WebBrowserWindow->OnBeforeBrowse().BindLambda([WeakBrowserWindow](const FString& Url, const FWebNavigationRequest& Request) -> bool
	{
		TSharedPtr<IWebBrowserWindow> BrowserWindow = WeakBrowserWindow.Pin();
		if (!BrowserWindow.IsValid())
		{
			return false;
		}

		if (!FabPluginSettings || FabPluginSettings->Environment == EFabEnvironment::CustomUrl)
		{
			return false;
		}

		return InterceptNavigation(BrowserWindow, Url, FabPluginSettings->GetUrlFromEnvironment());
	});
}

void FFabBrowser::InstallDebugPopup(const FFabBrowserTabContext& Context)
{
	if (!FabPluginSettings || !FabPluginSettings->bEnableDebugOptions)
	{
		return;
	}

	Context.WebBrowserWindow->OnCreateWindow().BindLambda(
		[](const TWeakPtr<IWebBrowserWindow>& NewBrowserWindow,
		   const TWeakPtr<IWebBrowserPopupFeatures>& /*PopupFeatures*/)
		{
			const TSharedRef<SWindow> Dialog = SNew(SWindow)
				.ClientSize(FVector2D(700, 700))
				.SupportsMaximize(true)
				.SupportsMinimize(true)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot().HAlign(HAlign_Fill).VAlign(VAlign_Fill)
					[
						SNew(SWebBrowser, NewBrowserWindow.Pin())
					]
				];

			FSlateApplication::Get().AddWindow(Dialog);
			return true;
		}
	);
}

void FFabBrowser::HandleTabActivated(TSharedRef<SDockTab> ActivatedTab, ETabActivationCause Cause, FName TabId)
{
	if (ActiveFabTabId == TabId)
	{
		return;
	}

	if (FFabBrowserTabContext* const& FabBrowserTabContext = TabsById.Find(TabId))
	{
		if (FabBrowserTabContext->bIsClosing)
		{
			return;
		}

		if (FabBrowserTabContext->WebBrowserWindow.IsValid())
		{
			FabBrowserTabContext->WebBrowserWindow->SetIsDisabled(false);
		}
	}

	ActiveFabTabId = TabId;
}

void FFabBrowser::HandleTabClosed(TSharedRef<SDockTab> ClosedTab, FName TabId)
{
	LogEvent(
		{
			"click",
			"button",
			"terminatePlugin",
			"closeFabPlugin",
			"interaction",
			{
				"Fab_UE5_Plugin",
				UFabBrowserApi::GetApiVersion()
			}
		}
	);

	FFabBrowserTabContext* Context = TabsById.Find(TabId);
	if (!Context)
	{
		return;
	}

	// Mark as closing immediately to guard other callbacks
	Context->bIsClosing = true;

	// Unbind JS bridge and clear browser callbacks synchronously
	// to prevent stale events during the 1.5s delay
	if (Context->WebBrowserInstance.IsValid())
	{
		if (UFabBrowserApi* Api = Context->JavascriptApi.Get())
		{
			Context->WebBrowserInstance->UnbindUObject(TEXT("fab"), Cast<UObject>(Api), true);
		}
	}

	if (Context->WebBrowserWindow.IsValid())
	{
		Context->WebBrowserWindow->OnUrlChanged().Clear();
		Context->WebBrowserWindow->OnBeforeBrowse().Unbind();
		Context->WebBrowserWindow->OnUnhandledKeyUp().Unbind();
		Context->WebBrowserWindow->OnUnhandledKeyDown().Unbind();
		Context->WebBrowserWindow->OnCreateWindow().Unbind();
	}

	Context->DockTab.Reset();

	// Clear active if needed
	if (ActiveFabTabId == TabId)
	{
		ActiveFabTabId = NAME_None;
	}

#if PLATFORM_MAC && !WITH_CEF3
	FSlateApplication::Get().ToggleDisableLastDragOnDragEnter(false);
#endif

	// If we are shutting down, do cleanup immediately instead of deferring.
	// The deferred lambda would fire after Shutdown() has already emptied TabsById,
	// causing access to stale static data.
	if (bIsShuttingDown)
	{
		ReleaseJavascriptApi(*Context);

		Context->WebBrowserInstance.Reset();
		Context->WebBrowserWindow.Reset();

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TabId);
		TabsById.Remove(TabId);
		return;
	}

	// Defer final resource release so the frontend can register the close event
	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask, [TabId]()
	{
		FPlatformProcess::Sleep(1.5f);

		AsyncTask(ENamedThreads::GameThread, [TabId]()
		{
			// Guard against Shutdown() having run while we were sleeping
			if (bIsShuttingDown)
			{
				return;
			}

			FFabBrowserTabContext* DeferredContext = TabsById.Find(TabId);
			if (!DeferredContext)
			{
				return;
			}

			ReleaseJavascriptApi(*DeferredContext);

			DeferredContext->WebBrowserInstance.Reset();
			DeferredContext->WebBrowserWindow.Reset();

			FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TabId);
			TabsById.Remove(TabId);
		});
	});
}

FFabBrowserTabContext* FFabBrowser::GetActiveDockTabContext()
{
	return TabsById.Find(ActiveFabTabId);
}

TWeakObjectPtr<UFabBrowserApi> FFabBrowser::GetBrowserApi()
{
	FFabBrowserTabContext* FabBrowserTabContext = TabsById.Find(ActiveFabTabId);
	if (!FabBrowserTabContext)
	{
		return TWeakObjectPtr<UFabBrowserApi>();
	}

	return FabBrowserTabContext->JavascriptApi;
}

FString FFabBrowser::GetUrl()
{
	if (FabPluginSettings == nullptr)
	{
		return TEXT("https://www.fab.com/plugins/ue5");
	}

	switch (FabPluginSettings->Environment)
	{
		default: // fall through
		case EFabEnvironment::Prod:  // fall through
		case EFabEnvironment::Gamedev: // fall through
		case EFabEnvironment::Test:  // fall through
		{
			FString Url = FabPluginSettings->GetUrlFromEnvironment();
			Url += TEXT("/plugins/ue5");
			return Url;
		}
		case EFabEnvironment::CustomUrl:
		{
			return FabPluginSettings->CustomUrl;
		}
	}
}

const ISlateStyle& FFabBrowser::GetStyleSet()
{
	return *(SlateStyleSet.Get());
}

void FFabBrowser::Shutdown()
{
	bIsShuttingDown = true;

#if PLATFORM_WINDOWS && (ENGINE_MAJOR_VERSION > 5 || (ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION >= 7))
	if (WindowMoveMessageHandler.IsValid())
	{
		if (FSlateApplication::IsInitialized())
		{
			if (FWindowsApplication* WindowsApp = static_cast<FWindowsApplication*>(FSlateApplication::Get().GetPlatformApplication().Get()))
			{
				WindowsApp->RemoveMessageHandler(*WindowMoveMessageHandler);
			}
		}
		WindowMoveMessageHandler.Reset();
	}
#endif

	FSlateStyleRegistry::UnRegisterSlateStyle(*SlateStyleSet);

	// Close / cleanup any still-open tabs and remove spawners
	// Copy keys because HandleTabClosed/Remove will mutate TabsById.
	TArray<FName> TabIds;
	TabsById.GetKeys(TabIds);

	for (const FName& TabId : TabIds)
	{
		// If tab is still open, request it to close.
		if (FFabBrowserTabContext* Context = TabsById.Find(TabId))
		{
			if (TSharedPtr<SDockTab> Tab = Context->DockTab.Pin())
			{
				// This will trigger OnTabClosed -> HandleTabClosed (in normal cases)
				Tab->RequestCloseTab();
			}
			else
			{
				// Tab slate object already gone; do direct cleanup
				ReleaseJavascriptApi(*Context);
				Context->WebBrowserInstance.Reset();
				Context->WebBrowserWindow.Reset();
			}
		}

		// Always unregister the spawner (safe even if already unregistered)
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TabId);
	}

	// In case some OnTabClosed didn't fire (shutdown ordering), force-clear leftovers.
	for (auto& Pair : TabsById)
	{
		FFabBrowserTabContext& Context = Pair.Value;

		ReleaseJavascriptApi(Context);

		Context.WebBrowserInstance.Reset();
		Context.WebBrowserWindow.Reset();
		Context.DockTab.Reset();
	}

	// Clean up error tab spawner in case SpawnErrorTab was called
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FName(TEXT("FabError")));

	TabsById.Empty();
	ActiveFabTabId = NAME_None;
}

void FFabBrowser::NotifyAllTabsLogout()
{
	for (const TPair<FName, FFabBrowserTabContext>& Pair : TabsById)
	{
		if (UFabBrowserApi* Api = Pair.Value.JavascriptApi.Get())
		{
			Api->ExecuteOnLogoutCallback();
		}
	}
}

void FFabBrowser::NotifyAllTabsLogin(const FString& AccessToken)
{
	for (const TPair<FName, FFabBrowserTabContext>& Pair : TabsById)
	{
		if (UFabBrowserApi* Api = Pair.Value.JavascriptApi.Get())
		{
			Api->ExecuteOnLoginCallback(AccessToken);
		}
	}
}

void FFabBrowser::LoggedIn(const FString& InAccessToken)
{
	NotifyAllTabsLogin(InAccessToken);
}

void FFabBrowser::LogEvent(const FFabAnalyticsPayload& Payload)
{
	FString JSONPayload;
	FJsonObjectConverter::UStructToJsonObjectString(Payload, JSONPayload, 0, 0, 0, nullptr, false);

	const FFabBrowserTabContext* FabBrowserTabContext = TabsById.Find(ActiveFabTabId);
	if (!FabBrowserTabContext)
	{
		return;
	}

	if (UFabBrowserApi* Api = FabBrowserTabContext->JavascriptApi.Get())
	{
		Api->ExecuteOnLogEventCallback(JSONPayload);
	}
}

void FFabBrowser::ShowSettings()
{
	AsyncTask(ENamedThreads::Type::GameThread, []()
	{
		const TSharedRef<SWindow> Window = SNew(SWindow)
			.Title(LOCTEXT("FabSettingsLabel", "Fab Settings"))
			.ClientSize(FVector2D(600.f, 300.f))
			.SizingRule(ESizingRule::UserSized);

		TSharedPtr<SFabSettingsWindow> SettingsWindow;
		Window->SetContent(SAssignNew(SettingsWindow, SFabSettingsWindow).WidgetWindow(Window));
		TSharedPtr<SWindow> ParentWindow;
		if (FModuleManager::Get().IsModuleLoaded("MainFrame"))
		{
			const IMainFrameModule& MainFrame = FModuleManager::LoadModuleChecked<IMainFrameModule>("MainFrame");
			ParentWindow                      = MainFrame.GetParentWindow();
		}

		FSlateApplication::Get().AddModalWindow(Window, ParentWindow, false);
	});
}

void FFabBrowser::OpenURL(const FString& InURL)
{
	if (ActiveFabTabId != NAME_None)
	{
		if (FFabBrowserTabContext* const& FabBrowserTabContext = TabsById.Find(ActiveFabTabId))
		{
			if (!FabBrowserTabContext->WebBrowserWindow.IsValid())
			{
				UE_LOG(LogFab, Warning, TEXT("OpenURL: Active tab '%s' has no valid WebBrowserWindow (browser may still be attaching). URL not loaded: %s"), *ActiveFabTabId.ToString(), *InURL);
				return;
			}
			FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor").GetLevelEditorTabManager()->TryInvokeTab(FabBrowserTabContext->TabId);
			if (FabBrowserTabContext->WebBrowserWindow->GetUrl() != InURL)
			{
				FabBrowserTabContext->WebBrowserWindow->LoadURL(InURL);
			}
		}
	}
}

void FFabBrowser::OpenInNewTab(const FString& Url)
{
	CreateNewFabTab(Url);
}

static bool IsHttpOrHttpsUrl(const FString& Url)
{
	return Url.StartsWith(TEXT("http://"), ESearchCase::IgnoreCase) ||
		Url.StartsWith(TEXT("https://"), ESearchCase::IgnoreCase);
}

// Choose Fab base matching CurrentUrl (assumes FabBaseUrl has no www)
static FString GetMatchingFabBaseUrl(const FString& CurrentUrl, const FString& FabBaseUrlNoWww)
{
	FString BaseUrl = FabBaseUrlNoWww;
	BaseUrl.RemoveFromEnd(TEXT("/"));

	if (CurrentUrl.StartsWith(TEXT("https://www."), ESearchCase::IgnoreCase))
	{
		return BaseUrl.Replace(TEXT("https://"), TEXT("https://www."), ESearchCase::IgnoreCase);
	}
	if (CurrentUrl.StartsWith(TEXT("http://www."), ESearchCase::IgnoreCase))
	{
		return BaseUrl.Replace(TEXT("http://"), TEXT("http://www."), ESearchCase::IgnoreCase);
	}

	return BaseUrl;
}

// Build plugin URL from a Fab URL
static FString BuildFabPluginUrl(const FString& CurrentUrl, const FString& MatchedBaseUrl)
{
	FString Remainder = CurrentUrl.RightChop(MatchedBaseUrl.Len());
	if (Remainder.IsEmpty())
	{
		Remainder = TEXT("/");
	}
	else if (!Remainder.StartsWith(TEXT("/")))
	{
		Remainder = TEXT("/") + Remainder;
	}

	return MatchedBaseUrl + TEXT("/plugins/ue5") + Remainder;
}

bool FFabBrowser::InterceptNavigation(const TSharedPtr<IWebBrowserWindow>& BrowserWindow, const FString& CurrentUrl, const FString& FabBaseUrl)
{
	if (!BrowserWindow.IsValid())
	{
		return false;
	}

	if (!IsHttpOrHttpsUrl(CurrentUrl))
	{
		return false;
	}

	const FString BaseUrl = GetMatchingFabBaseUrl(CurrentUrl, FabBaseUrl);
	const bool bIsFabUrl = CurrentUrl.StartsWith(BaseUrl, ESearchCase::IgnoreCase);

	// Only intercept Fab URLs; let everything else (payment providers, SSO, etc.) load normally
	if (!bIsFabUrl)
	{
		return false;
	}

	// Already a plugin Fab URL, can ignore
	if (CurrentUrl.Contains(TEXT("/plugins/ue5"), ESearchCase::IgnoreCase))
	{
		return false;
	}

	// Don't remap 3D previewer URLs
	if (CurrentUrl.Contains(TEXT("/dope/"), ESearchCase::IgnoreCase))
	{
		return false;
	}

	// Don't remap payment URLs
	if (CurrentUrl.Contains(TEXT("/payment/"), ESearchCase::IgnoreCase))
	{
		return false;
	}

	// Remap Fab web URL to plugin URL
	const FString Remapped = BuildFabPluginUrl(CurrentUrl, BaseUrl);
	if (!Remapped.Equals(CurrentUrl, ESearchCase::CaseSensitive))
	{
		FAB_LOG("Loading remapped URL [%s]", *Remapped);
		BrowserWindow->LoadURL(Remapped);
		return true; // cancel original navigation
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
