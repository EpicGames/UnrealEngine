// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ContentBrowserDelegates.h"
#include "FabBrowserApi.h"
#include "FabBrowser.generated.h"

class SDockTab;
class SWebBrowser;
class FSpawnTabArgs;
class IWebBrowserWindow;
class UFabBrowserApi;
class UFabSettings;
class FSlateStyleSet;
class FExtender;

USTRUCT()
struct FFabAnalyticsEventValue
{
	GENERATED_BODY()

	UPROPERTY()
	FString Platform;

	UPROPERTY()
	FFabApiVersion ApiVersion;
};

USTRUCT()
struct FFabAnalyticsPayload
{
	GENERATED_BODY()

	UPROPERTY()
	FString InteractionType;

	UPROPERTY()
	FString EventCategory;

	UPROPERTY()
	FString EventAction;

	UPROPERTY()
	FString EventLabel;

	UPROPERTY()
	FString EventType;

	UPROPERTY()
	FFabAnalyticsEventValue EventValue;
};

struct FFabBrowserTabContext
{
	FName TabId = NAME_None;

	// Slate/CEF objects
	TWeakPtr<SDockTab> DockTab;
	TSharedPtr<IWebBrowserWindow> WebBrowserWindow;
	TSharedPtr<SWebBrowser> WebBrowserInstance;

	// JS bridge
	// Held as a weak pointer so access after GC returns null instead of dereferencing a
	// dangling address. Liveness while in use is anchored by AddToRoot / RemoveFromRoot.
	// Game-thread only: TWeakObjectPtr is not thread-safe.
	TWeakObjectPtr<UFabBrowserApi> JavascriptApi;

	bool bIsNativelyRendered = false;
	bool bIsClosing = false;
};

class FFabBrowser
{
private:
	static TUniquePtr<FSlateStyleSet> SlateStyleSet;
	static TObjectPtr<const UFabSettings> FabPluginSettings;

	// Tabs
	static int32 FabTabCounter;
	static FName ActiveFabTabId;
	static TMap<FName, FFabBrowserTabContext> TabsById;
	static TSharedPtr<FTabManager> TabManager;
	static bool bIsShuttingDown;

	static const FText FabLabel;
	static const FText FabDefaultTabTitle;
	static const FText FabTooltip;
	static const FName FabMenuIconName;
	static const FName FabAssetIconName;
	static const FName FabToolbarIconName;

	static void RegisterSlateStyle();
	static void ExtendContextMenuInContentBrowser();
	static void SetupEntryPoints();

	// Helpers
	static FName MakeNextFabTabId();
	static bool EnsureWebBrowserAvailable();
	static FString BuildIndexFileUrl();
	static void PreventBrowserKeyConflictWithEditor(const FFabBrowserTabContext& Context);
	static void InstallUrlRestrictions(const FFabBrowserTabContext& Context);
	static void InstallDebugPopup(const FFabBrowserTabContext& Context);
	static bool InterceptNavigation(const TSharedPtr<IWebBrowserWindow>& BrowserWindow, const FString& CurrentUrl, const FString& FabBaseUrl);
	static void SetupDockTabListenersForNativeRendering(const TSharedRef<SDockTab>& DockTab, const FName TabId);
	static void SetupTabWindowMoveListenerForNativeRendering(const TSharedRef<SDockTab>& DockTab);
	static void ReparentDockTabs();
	static void RefreshNativeTabViewPort();

	// Tab Creation
	static void CreateNewFabTab(const FString& Url = TEXT(""));
	static void CreateOsrFabTab(const FString& Url);
	static void PreventWebBrowserFreezeInOsrMode(const FName& TabId, const FFabBrowserTabContext& Context);
	static void CreateNativeFabTab(const FString& Url);
	static void AttachBrowserToDockTab(const TSharedRef<SDockTab>& DockTab, const bool bNativeRendering, const FName TabId, const FString& Url);
	static void HandleTabClosed(TSharedRef<SDockTab> ClosedTab, FName TabId);
	static void HandleTabActivated(TSharedRef<SDockTab> ActivatedTab, ETabActivationCause Cause, FName TabId);

	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);
	static TSharedRef<SWidget> OnFabAssetIconGenerate(const FAssetData& AssetData);

public:
	static void Init();
	static void Shutdown();

	/** Notify all open Fab tabs that a logout has occurred by invoking their OnLogout JS callback. */
	static void NotifyAllTabsLogout();

	/** Notify all open Fab tabs that a login has occurred by sending the access token via their OnLogin JS callback. */
	static void NotifyAllTabsLogin(const FString& AccessToken);

	static void LogEvent(const FFabAnalyticsPayload& Payload);
	static void LoggedIn(const FString& InAccessToken);
	static TWeakObjectPtr<UFabBrowserApi> GetBrowserApi();
	static void ShowSettings();
	static void OpenURL(const FString& InURL = GetUrl());
	/** Opens the given URL in a new Fab plugin tab (native webview, not the system browser). */
	static void OpenInNewTab(const FString& Url);
	static FFabBrowserTabContext* GetActiveDockTabContext();
	static FString GetUrl();
	static const ISlateStyle& GetStyleSet();
};
