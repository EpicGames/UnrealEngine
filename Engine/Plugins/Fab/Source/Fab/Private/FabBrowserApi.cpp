// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabBrowserApi.h"
#include "FabAuthentication.h"
#include "FabBrowser.h"
#include "FabSettings.h"
#include "FabLog.h"
#include "FabWorkflowFactoryRegistry.h"

#include "HAL/PlatformApplicationMisc.h"
#include "Interfaces/IPluginManager.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/EngineVersion.h"
#include "Workflows/GenericImportWorkflow.h"
#include "Workflows/PackImportWorkflow.h"
#include "Workflows/QuixelImportWorkflow.h"
#include "Workflows/GenericDragDropWorkflow.h"
#include "Workflows/MetaHumanImportWorkflow.h"
#include "Workflows/PluginInstallWorkflow.h"
#include "Workflows/QuixelDragDropWorkflow.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FabBrowserApi)

void UFabBrowserApi::CompleteWorkflow(const FString& Id)
{
	TSharedPtr<IFabWorkflow> CompletedWorkflow;
	for (const TSharedPtr<IFabWorkflow>& ActiveWorkflow : this->ActiveWorkflows)
	{
		if (ActiveWorkflow->AssetId == Id)
		{
			CompletedWorkflow = ActiveWorkflow;
			break;
		}
	}

	this->ActiveWorkflows.Remove(CompletedWorkflow);
}

void UFabBrowserApi::RegisterCallbacks(const FFabCallbacks& Callbacks)
{
	OnLogoutDelegate.Unbind();
	OnLoginDelegate.Unbind();
	OnLogEventDelegate.Unbind();

	if (Callbacks.OnLogout.IsValid())
	{
		OnLogoutDelegate.BindLambda(Callbacks.OnLogout);
	}

	if (Callbacks.OnLogin.IsValid())
	{
		OnLoginDelegate.BindLambda(Callbacks.OnLogin);
	}

	if (Callbacks.OnLogEvent.IsValid())
	{
		OnLogEventDelegate.BindLambda(Callbacks.OnLogEvent);
	}
}

void UFabBrowserApi::ExecuteOnLogoutCallback() const
{
	OnLogoutDelegate.ExecuteIfBound(0);
}

void UFabBrowserApi::ExecuteOnLoginCallback(const FString& AccessToken) const
{
	OnLoginDelegate.ExecuteIfBound(AccessToken);
}

void UFabBrowserApi::ExecuteOnLogEventCallback(const FString& JSONPayload) const
{
	OnLogEventDelegate.ExecuteIfBound(JSONPayload);
}

void UFabBrowserApi::AddToProject(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata)
{
	// Check if the listing is already being downloaded
	for (const TSharedPtr<IFabWorkflow>& ActiveWorkflow : this->ActiveWorkflows)
	{
		if (ActiveWorkflow->AssetId == AssetMetadata.AssetId)
		{
			FAB_LOG("The listing with Id %s is already being processed.", *AssetMetadata.AssetId);
			return;
		}
	}

	FAB_LOG("Add listing to project - %s", *AssetMetadata.AssetId);
	FAB_LOG("Listing type - %s", *AssetMetadata.AssetType);

	if (AssetMetadata.AssetType == "unreal-engine")
	{
		const FString BaseUrls = FString::Join(AssetMetadata.DistributionPointBaseUrls, TEXT(","));
		FAB_LOG("Base Url %s", *BaseUrls);

		FString FinalDownloadUrl = DownloadUrl;
		if (DownloadUrl.StartsWith(TEXT("\\\\")))
		{
			FinalDownloadUrl = DownloadUrl.Replace(TEXT("\\\\"), TEXT("file://"));
		}

		const TSharedPtr<FPackImportWorkflow> PackImportWorkflow = MakeShared<FPackImportWorkflow>(
			AssetMetadata.AssetId, AssetMetadata.AssetName, FinalDownloadUrl, BaseUrls);
		PackImportWorkflow->OnFabWorkflowComplete().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		PackImportWorkflow->OnFabWorkflowCancel().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		
		AsyncTask(ENamedThreads::Type::GameThread, [PackImportWorkflow]() 
	    {
			PackImportWorkflow->Execute();
		});
		
		this->ActiveWorkflows.Add(PackImportWorkflow);
		return;
	}

	if (AssetMetadata.IsQuixel)
	{
		const TSharedPtr<FQuixelImportWorkflow> QuixelImportWorkflow = MakeShared<FQuixelImportWorkflow>(
			AssetMetadata.AssetId, AssetMetadata.AssetName, DownloadUrl);
		QuixelImportWorkflow->Execute();
		QuixelImportWorkflow->OnFabWorkflowComplete().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		QuixelImportWorkflow->OnFabWorkflowCancel().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		this->ActiveWorkflows.Add(QuixelImportWorkflow);
		return;
	}

	if (AssetMetadata.AssetType == "gltf" || AssetMetadata.AssetType == "glb" || AssetMetadata.AssetType == "fbx")
	{
		const TSharedPtr<FGenericImportWorkflow> InterchangeImportWorkflow = MakeShared<FGenericImportWorkflow>(
			AssetMetadata.AssetId, AssetMetadata.AssetName, DownloadUrl);
		InterchangeImportWorkflow->OnFabWorkflowComplete().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		InterchangeImportWorkflow->OnFabWorkflowCancel().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		InterchangeImportWorkflow->Execute();
		this->ActiveWorkflows.Add(InterchangeImportWorkflow);
		return;
	}

	if (AssetMetadata.AssetType == "metahuman")
	{
		const TSharedPtr<FMetaHumanImportWorkflow> MetaHumanImportWorkflow = MakeShared<FMetaHumanImportWorkflow>(
			AssetMetadata.AssetId, AssetMetadata.AssetName, DownloadUrl);
		MetaHumanImportWorkflow->OnFabWorkflowComplete().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		MetaHumanImportWorkflow->OnFabWorkflowCancel().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		MetaHumanImportWorkflow->Execute();
		this->ActiveWorkflows.Add(MetaHumanImportWorkflow);
		return;
	}

	if (const TSharedPtr<IFabWorkflowFactory> ImportWorkflowFactory = FFabWorkflowFactoryRegistry::GetFactory(AssetMetadata.AssetType))
	{
		const TSharedPtr<IFabWorkflow> ImportWorkflow = ImportWorkflowFactory->Create(AssetMetadata, DownloadUrl);
		ImportWorkflow->OnFabWorkflowComplete().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		ImportWorkflow->OnFabWorkflowCancel().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				CompleteWorkflow(AssetId);
			}
		);
		ImportWorkflow->Execute();
		this->ActiveWorkflows.Add(ImportWorkflow);
		return;
	}

	FAB_LOG_ERROR("Asset type not handled %s", *AssetMetadata.AssetType);
}

void UFabBrowserApi::InstallToProject(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata)
{
	for (const TSharedPtr<IFabWorkflow>& ActiveWorkflow : this->ActiveWorkflows)
	{
		if (ActiveWorkflow->AssetId == AssetMetadata.AssetId)
		{
			FAB_LOG("The Plugin with Id %s is already being processed.", *AssetMetadata.AssetId);
			return;
		}
	}

	FAB_LOG("Install plugin to project - %s", *AssetMetadata.AssetId);

	FString FinalDownloadUrl = DownloadUrl;
	if (DownloadUrl.StartsWith(TEXT("\\\\")))
	{
		FinalDownloadUrl = DownloadUrl.Replace(TEXT("\\\\"), TEXT("file://"));
	}

	const FString BaseUrls = FString::Join(AssetMetadata.DistributionPointBaseUrls, TEXT(","));
	const TSharedPtr<FPluginInstallWorkflow> PluginInstallWorkflow = MakeShared<FPluginInstallWorkflow>(AssetMetadata.AssetId, AssetMetadata.AssetName, FinalDownloadUrl, BaseUrls, false);
	PluginInstallWorkflow->OnFabWorkflowComplete().BindLambda(
		[this, AssetId = AssetMetadata.AssetId]()
		{
			CompleteWorkflow(AssetId);
		}
	);
	PluginInstallWorkflow->OnFabWorkflowCancel().BindLambda(
		[this, AssetId = AssetMetadata.AssetId]()
		{
			CompleteWorkflow(AssetId);
		}
	);

	PluginInstallWorkflow->Execute();
	this->ActiveWorkflows.Add(PluginInstallWorkflow);
}

void UFabBrowserApi::InstallToEngine(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata)
{
	for (const TSharedPtr<IFabWorkflow>& ActiveWorkflow : this->ActiveWorkflows)
	{
		if (ActiveWorkflow->AssetId == AssetMetadata.AssetId)
		{
			FAB_LOG("The Plugin with Id %s is already being processed.", *AssetMetadata.AssetId);
			return;
		}
	}

	FAB_LOG("Install plugin to engine - %s", *AssetMetadata.AssetId);

	FString FinalDownloadUrl = DownloadUrl;
	if (DownloadUrl.StartsWith(TEXT("\\\\")))
	{
		FinalDownloadUrl = DownloadUrl.Replace(TEXT("\\\\"), TEXT("file://"));
	}

	const FString BaseUrls = FString::Join(AssetMetadata.DistributionPointBaseUrls, TEXT(","));
	const TSharedPtr<FPluginInstallWorkflow> PluginInstallWorkflow = MakeShared<FPluginInstallWorkflow>(AssetMetadata.AssetId, AssetMetadata.AssetName, FinalDownloadUrl, BaseUrls, true);
	PluginInstallWorkflow->OnFabWorkflowComplete().BindLambda(
		[this, AssetId = AssetMetadata.AssetId]()
		{
			CompleteWorkflow(AssetId);
		}
	);
	PluginInstallWorkflow->OnFabWorkflowCancel().BindLambda(
		[this, AssetId = AssetMetadata.AssetId]()
		{
			CompleteWorkflow(AssetId);
		}
	);

	PluginInstallWorkflow->Execute();
	this->ActiveWorkflows.Add(PluginInstallWorkflow);
}

void UFabBrowserApi::DragStart(const FFabAssetMetadata& AssetMetadata)
{
	// Check if the listing is already being downloaded
	for (const TSharedPtr<IFabWorkflow>& ActiveWorkflow : this->ActiveWorkflows)
	{
		if (ActiveWorkflow->AssetId == AssetMetadata.AssetId)
		{
			FAB_LOG("The listing with Id %s is already being processed.", *AssetMetadata.AssetId);
			return;
		}
	}

	FAB_LOG("Drag listing to project - %s", *AssetMetadata.AssetId);
	FAB_LOG("Listing type - %s", *AssetMetadata.AssetType);

	if (AssetMetadata.IsQuixel)
	{
		const TSharedPtr<FQuixelDragDropWorkflow> QuixelDragDropWorkflow = MakeShared<FQuixelDragDropWorkflow>(
			AssetMetadata.AssetId,
			AssetMetadata.AssetName,
			AssetMetadata.ListingType
		);
		QuixelDragDropWorkflow->OnFabWorkflowComplete().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				FAB_LOG("Quixel Drag workflow completed!");
				CompleteWorkflow(AssetId);
			}
		);
		QuixelDragDropWorkflow->OnFabWorkflowCancel().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				FAB_LOG("Quixel Drag workflow cancelled!");
				CompleteWorkflow(AssetId);
			}
		);
		this->ActiveWorkflows.Add(QuixelDragDropWorkflow);
		QuixelDragDropWorkflow->Execute();
	}
	else
	{
		const TSharedPtr<FGenericDragDropWorkflow> DragDropWorkflow = MakeShared<FGenericDragDropWorkflow>(
			AssetMetadata.AssetId, AssetMetadata.AssetName);
		DragDropWorkflow->OnFabWorkflowComplete().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				FAB_LOG("Drag workflow completed!");
				CompleteWorkflow(AssetId);
			}
		);
		DragDropWorkflow->OnFabWorkflowCancel().BindLambda(
			[this, AssetId = AssetMetadata.AssetId]()
			{
				FAB_LOG("Drag workflow cancelled!");
				CompleteWorkflow(AssetId);
			}
		);
		this->ActiveWorkflows.Add(DragDropWorkflow);
		DragDropWorkflow->Execute();
	}
}

void UFabBrowserApi::OnDragInfoSuccess(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata)
{
	OnSignedUrlGenerated().Broadcast(DownloadUrl, AssetMetadata);
}

void UFabBrowserApi::OnDragInfoFailure(const FString& AssetId)
{
	FAB_LOG_ERROR("Drag drop failure for asset id %s", *AssetId);
	// Get the drag workflow
	FFabAssetMetadata Metadata;
	Metadata.AssetId = AssetId;
	OnSignedUrlGenerated().Broadcast("", Metadata);
}

FDelegateHandle UFabBrowserApi::AddSignedUrlCallback(TFunction<void(const FString&, const FFabAssetMetadata&)> Callback)
{
	return OnSignedUrlGenerated().AddLambda(Callback);
}

void UFabBrowserApi::Login()
{
	FabAuthentication::LoginUsingAccountPortal();
}

void UFabBrowserApi::Logout()
{
	FabAuthentication::DeletePersistentAuth();
	FFabBrowser::NotifyAllTabsLogout();
}

void UFabBrowserApi::OpenPluginSettings()
{
	FFabBrowser::ShowSettings();
}

FFabFrontendSettings UFabBrowserApi::GetSettings()
{
	const UFabSettings* FabSettings = GetDefault<UFabSettings>();

	FFabFrontendSettings FrontendSettings;
	if (FabSettings->PreferredDefaultFormat == EFabPreferredFormats::GLTF)
	{
		FrontendSettings.preferredformat = "gltf";
	}
	else if (FabSettings->PreferredDefaultFormat == EFabPreferredFormats::FBX)
	{
		FrontendSettings.preferredformat = "fbx";
	}
	if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::Low)
	{
		FrontendSettings.preferredquality = "low";
	}
	else if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::Medium)
	{
		FrontendSettings.preferredquality = "medium";
	}
	else if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::High)
	{
		FrontendSettings.preferredquality = "high";
	}
	else if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::Raw)
	{
		FrontendSettings.preferredquality = "raw";
	}

	return FrontendSettings;
}

void UFabBrowserApi::SetPreferredQualityTier(const FString& preferredquality)
{
	UFabSettings* FabSettings = GetMutableDefault<UFabSettings>();
	if (preferredquality == "low")
	{
		FabSettings->PreferredQualityTier = EFabPreferredQualityTier::Low;
	}
	else if (preferredquality == "medium")
	{
		FabSettings->PreferredQualityTier = EFabPreferredQualityTier::Medium;
	}
	else if (preferredquality == "high")
	{
		FabSettings->PreferredQualityTier = EFabPreferredQualityTier::High;
	}
	else if (preferredquality == "raw")
	{
		FabSettings->PreferredQualityTier = EFabPreferredQualityTier::Raw;
	}

	FabSettings->SaveConfig();
}

FFabApiVersion UFabBrowserApi::GetApiVersion()
{
	FFabApiVersion ApiVersion;
	const FEngineVersion EngineVersion = FEngineVersion::Current();
	const uint16 MajorVersion = EngineVersion.GetMajor();
	const uint16 MinorVersion = EngineVersion.GetMinor();

	ApiVersion.ue = FString::FromInt(MajorVersion) + "." + FString::FromInt(MinorVersion);
	ApiVersion.api = "1.1.0";

	if (const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin("Fab"); Plugin.IsValid())
	{
		const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();

		const FString PluginVersion = PluginDescriptor.VersionName;
		ApiVersion.pluginversion = PluginVersion;
	}

	ApiVersion.platform = UGameplayStatics::GetPlatformName();

	// Supported features
	FFabSupportedFeatures SupportedFeatures;
	SupportedFeatures.banners = true;

	ApiVersion.supportedfeatures = SupportedFeatures;

	return ApiVersion;
}

void UFabBrowserApi::OpenInNewTab(const FString& Url)
{
	FAB_LOG("OpenInNewTab %s", *Url);
	FFabBrowser::OpenInNewTab(Url);
}

void UFabBrowserApi::OpenUrlInBrowser(const FString& Url)
{
	FAB_LOG("OpenUrlInBrowser %s", *Url);
	FPlatformProcess::LaunchURL(*Url, nullptr, nullptr);
}

void UFabBrowserApi::CopyToClipboard(const FString& Content)
{
	FPlatformApplicationMisc::ClipboardCopy(*Content);
}

void UFabBrowserApi::PluginOpened()
{
	FFabBrowser::LogEvent(
		{
			"click",
			"button",
			"startPlugin",
			"openFabPlugin",
			"interaction",
			{
				"Fab_UE5_Plugin",
				GetApiVersion()
			}
		}
	);
}

FString UFabBrowserApi::GetUrl()
{
	return FFabBrowser::GetUrl();
}

FString UFabBrowserApi::GetAuthToken()
{
	const UFabSettings* FabSettings = GetDefault<UFabSettings>();
	if (!FabSettings->CustomAuthToken.IsEmpty())
	{
		FAB_LOG("Returning custom auth token: %s", *FabSettings->CustomAuthToken);
		return FabSettings->CustomAuthToken;
	}

	return FabAuthentication::GetAuthToken();
}

FString UFabBrowserApi::GetRefreshToken()
{
	return FabAuthentication::GetRefreshToken();
}
