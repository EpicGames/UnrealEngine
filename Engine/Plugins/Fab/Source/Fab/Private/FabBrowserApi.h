// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WebJSFunction.h"

#include "Workflows/FabWorkflow.h"

#include "FabBrowserApi.generated.h"

USTRUCT()
struct FFabSupportedFeatures
{
	GENERATED_BODY()

	// These uproperties need to be in this case (NOT PascalCase)
	// otherwise we will break compatibility with the engine plugin
	// Fab's JS on the macOS WebKit WebBrowser plugin backend.

	UPROPERTY()
	bool banners = false;
};

USTRUCT()
struct FFabApiVersion
{
	GENERATED_BODY()

	// These uproperties need to be in this case (NOT PascalCase)
	// otherwise we will break compatibility with the engine plugin
	// Fab's JS on the macOS WebKit WebBrowser plugin backend.

	UPROPERTY()
	FString ue;

	UPROPERTY()
	FString api;

	UPROPERTY()
	FString pluginversion;

	UPROPERTY()
	FString platform;

	UPROPERTY()
	FFabSupportedFeatures supportedfeatures;
};

USTRUCT()
struct FFabFrontendSettings
{
	GENERATED_BODY()

	// These two uproperties need to be in this case (NOT PascalCase)
	// otherwise we will break compatibility with the engine plugin
	// Fab's JS on the macOS WebKit WebBrowser plugin backend.
	UPROPERTY()
	FString preferredformat;

	UPROPERTY()
	FString preferredquality;
};

USTRUCT()
struct FFabCallbacks
{
	GENERATED_BODY()

	UPROPERTY()
	FWebJSFunction OnLogout;

	UPROPERTY()
	FWebJSFunction OnLogin;

	UPROPERTY()
	FWebJSFunction OnLogEvent;
};

UCLASS()
class UFabBrowserApi : public UObject
{
	GENERATED_BODY()
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSignedUrlGenerated, const FString& /*DownloadUrl*/, FFabAssetMetadata /*Metadata*/);

	// Frontend callbacks
	DECLARE_DELEGATE_OneParam(FOnLogout, int32 /* dummy value */);
	DECLARE_DELEGATE_OneParam(FOnLogin, const FString& /* AccessToken */);
	DECLARE_DELEGATE_OneParam(FOnLogEvent, const FString& /* JSONPayload */);

private:
	FOnSignedUrlGenerated OnSignedUrlGeneratedDelegate;
	void CompleteWorkflow(const FString& Id);
	TArray<TSharedPtr<IFabWorkflow>> ActiveWorkflows;

	FOnLogout OnLogoutDelegate;
	FOnLogin OnLoginDelegate;
	FOnLogEvent OnLogEventDelegate;

public:
	UFUNCTION()
	void RegisterCallbacks(const FFabCallbacks& Callbacks);

	/** Invoke the stored OnLogout callback, if one has been registered by the frontend. */
	void ExecuteOnLogoutCallback() const;

	/** Invoke the stored OnLogin callback, if one has been registered by the frontend. */
	void ExecuteOnLoginCallback(const FString& AccessToken) const;

	/** Invoke the stored OnLogEvent callback, if one has been registered by the frontend. */
	void ExecuteOnLogEventCallback(const FString& JSONPayload) const;

	UFUNCTION()
	void AddToProject(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata);

	UFUNCTION()
	void InstallToProject(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata);

	UFUNCTION()
	void InstallToEngine(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata);

	UFUNCTION()
	void DragStart(const FFabAssetMetadata& AssetMetadata);

	UFUNCTION()
	void OnDragInfoSuccess(const FString& DownloadUrl, const FFabAssetMetadata& AssetMetadata);

	UFUNCTION()
	void OnDragInfoFailure(const FString& AssetId);

	UFUNCTION()
	void Login();

	UFUNCTION()
	void Logout();

	UFUNCTION()
	FString GetAuthToken();

	UFUNCTION()
	FString GetRefreshToken();

	UFUNCTION()
	void OpenPluginSettings();

	UFUNCTION()
	FFabFrontendSettings GetSettings();

	UFUNCTION()
	void SetPreferredQualityTier(const FString& PreferredQuality);

	UFUNCTION()
	static FFabApiVersion GetApiVersion();

	UFUNCTION()
	void OpenInNewTab(const FString& Url);

	UFUNCTION()
	void OpenUrlInBrowser(const FString& Url);

	UFUNCTION()
	void CopyToClipboard(const FString& Content);

	UFUNCTION()
	void PluginOpened();

	UFUNCTION()
	FString GetUrl();

	FDelegateHandle AddSignedUrlCallback(TFunction<void(const FString&, const FFabAssetMetadata&)> Callback);
	FOnSignedUrlGenerated& OnSignedUrlGenerated() { return OnSignedUrlGeneratedDelegate; }
	void RemoveSignedUrlHandle(const FDelegateHandle& Handle) { OnSignedUrlGeneratedDelegate.Remove(Handle); }
};
