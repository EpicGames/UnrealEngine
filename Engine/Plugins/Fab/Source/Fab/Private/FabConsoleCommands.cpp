// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabAuthentication.h"
#include "FabBrowser.h"
#include "FabLog.h"
#include "FabSettings.h"
#include "Teds/FabMyFolderIntegration.h"

#include "HAL/IConsoleManager.h"
#include "Utilities/FabAssetsCache.h"

static FAutoConsoleCommand ConsoleCmd_FabShowSettings(
	TEXT("Fab.ShowSettings"),
	TEXT("Display the Fab settings window"),
	FConsoleCommandDelegate::CreateStatic(&FFabBrowser::ShowSettings)
);

static FAutoConsoleCommand ConsoleCmd_FabLogout(
	TEXT("Fab.Logout"),
	TEXT("Trigger a manual logout for Fab plugin"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FabAuthentication::DeletePersistentAuth();
	})
);

static FAutoConsoleCommand ConsoleCmd_FabLogin(
	TEXT("Fab.Login"),
	TEXT("Trigger a manual login for Fab plugin"),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FabAuthentication::LoginUsingAccountPortal();
	})
);

static FAutoConsoleCommand ConsoleCmd_FabClearCache(
	TEXT("Fab.ClearCache"),
	TEXT("Clear download cache for Fab plugin"),
	FConsoleCommandDelegate::CreateStatic(&FFabAssetsCache::ClearCache)
);

static FAutoConsoleCommand ConsoleCmd_FabSetEnvironment(
	TEXT("Fab.SetEnvironment"),
	TEXT("Set Fab plugin environment"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.IsEmpty())
		{
			FAB_LOG("Need to provide a valid environment arg");
			return;
		}
		
		const FString Environment = Args[0];
		UFabSettings* FabSettings = GetMutableDefault<UFabSettings>();
		if (Environment == "prod")
		{
			FabSettings->Environment = EFabEnvironment::Prod;
		}
		else if (Environment == "gamedev")
		{
			FabSettings->Environment = EFabEnvironment::Gamedev;
		}
		else if (Environment == "test")
		{
			FabSettings->Environment = EFabEnvironment::Test;
		}
		else if (Environment == "custom-url")
		{
			FabSettings->Environment = EFabEnvironment::CustomUrl;
		}

		FabAuthentication::DeletePersistentAuth();
		FabSettings->SaveConfig();
	})
);

static FAutoConsoleCommand ConsoleCmd_FabTedsMyFolderIntegration(
	TEXT("Fab.TEDS.MyFolderIntegration"),
	TEXT("Loads the content of My Folder into TEDS for use in the editor, e.g. by the Content Browser."),
	FConsoleCommandWithArgsDelegate::CreateLambda(
		[](const TArray<FString>& Args)
		{
			if (!Args.IsEmpty())
			{
				if (int32 BatchSize = FCString::Atoi(*(Args[0])); BatchSize > 0)
				{
					FFabTedsMyFolderIntegration::QueueSyncRequest(static_cast<uint32>(BatchSize));
					return;
				}
			}
			FFabTedsMyFolderIntegration::QueueSyncRequest();
		})
);
