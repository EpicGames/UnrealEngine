// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabDownloader.h"

#include "FabLog.h"
#include "HttpModule.h"
#include "HAL/PlatformFileManager.h"

#include "Importers/BuildPatchInstallerLibHelper.h"

#include "Interfaces/IHttpResponse.h"
#include "Interfaces/IPluginManager.h"

#include "Misc/DateTime.h"
#include "Misc/Timespan.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

#include "Runtime/Launch/Resources/Version.h"

#include "Utilities/FabAssetsCache.h"

TUniquePtr<BpiLib::IBpiLib> FFabDownloadRequest::BuildPatchServices;
FTSTicker::FDelegateHandle FFabDownloadRequest::BpsTickerHandle;

FFabDownloadRequest::FFabDownloadRequest(const FString& InAssetID, const FString& InDownloadURL, const FString& InDownloadLocation, EFabDownloadType InDownloadType)
	: AssetID(InAssetID)
	, DownloadURL(InDownloadURL)
	, DownloadLocation(InDownloadLocation)
	, DownloadType(InDownloadType)
{}

bool FFabDownloadRequest::LoadBuildPatchServices()
{
	if (BuildPatchServices)
	{
		return true;
	}

	constexpr auto WinLibName   = TEXT("BuildPatchInstallerLib.dll");
	constexpr auto LinuxLibName = TEXT("libBuildPatchInstallerLib.so");
	constexpr auto MacArmLibName   = TEXT("BuildPatchInstallerLib-arm.dylib");
	constexpr auto Macx86LibName   = TEXT("BuildPatchInstallerLib-x86.dylib");
	constexpr auto DLLName      = PLATFORM_WINDOWS ? WinLibName : PLATFORM_LINUX ? LinuxLibName : PLATFORM_MAC_ARM64 ? MacArmLibName : PLATFORM_MAC_X86 ? Macx86LibName : nullptr;
	if constexpr (DLLName == nullptr)
	{
		return false;
	}

	const FString PluginPath = IPluginManager::Get().FindPlugin(TEXT("Fab"))->GetBaseDir();
	const FString LibPath    = FPaths::ConvertRelativePathToFull(FPaths::Combine(PluginPath, TEXT("ThirdParty"), DLLName));

	BuildPatchServices = BpiLib::FBpiLibHelperFactory::Create(LibPath);
	if (BuildPatchServices)
	{
		BpsTickerHandle = FTSTicker::GetCoreTicker().AddTicker(
			FTickerDelegate::CreateLambda(
				[](const float Delta)
				{
					BuildPatchServices->Tick(Delta);
					return true;
				}
			)
		);
	}

	return BuildPatchServices != nullptr;
}

void FFabDownloadRequest::ShutdownBpsModule()
{
	if (BpsTickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(BpsTickerHandle);
	}
	if (BuildPatchServices.IsValid())
	{
		BuildPatchServices.Reset();
	}
}

void FFabDownloadRequest::ExecuteHTTPRequest()
{
	const FString FullFileName = GetFilenameFromURL(DownloadURL);
	const FString SaveFilename = DownloadLocation / FullFileName;
	const FString DownloadFile = SaveFilename + TEXT(".download");

	FHttpModule& HTTPModule = FHttpModule::Get();

	DownloadRequest = HTTPModule.CreateRequest().ToSharedPtr();
	DownloadRequest->SetURL(DownloadURL);
	DownloadRequest->OnHeaderReceived().BindLambda(
		[this, FullFileName](FHttpRequestPtr Request, const FString& HeaderName, const FString& HeaderValue)
		{
			if (HeaderName == "Content-Length")
			{
				DownloadStats.TotalBytes    = FCString::Atoi64(*HeaderValue);
				const FString CachedAssetId = AssetID / FullFileName;
				OnDownloadProgressDelegate.Broadcast(this, DownloadStats);

				if (FFabAssetsCache::IsCached(CachedAssetId, DownloadStats.TotalBytes))
				{
					DownloadStats.PercentComplete     = 100.0f;
					DownloadStats.DownloadCompletedAt = FDateTime::Now().ToUnixTimestamp();
					DownloadStats.bIsSuccess          = true;
					DownloadStats.bIsCached           = true;
					DownloadStats.CompletedBytes      = DownloadStats.TotalBytes;
					DownloadStats.DownloadedFiles     = {
						FFabAssetsCache::GetCachedFile(CachedAssetId)
					};

					Request->CancelRequest();
				}
			}
		}
	);

	// we need to create dir path to store the downloaded file
	const FString Directory = FPaths::GetPath(DownloadFile);
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	if (!PlatformFile.DirectoryExists(*Directory))
	{
		if (!PlatformFile.CreateDirectoryTree(*Directory))
		{
			FAB_LOG_ERROR("Failed to create the dir for writing: %s", *Directory);
			DownloadStats.bIsSuccess = false;
			OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
			return;
		}
	}

	// stream directly to file
	struct FFileHandleWrapper { TUniquePtr<IFileHandle> FileHandle; };
	TSharedPtr<FFileHandleWrapper> StreamToFileHandle = MakeShared<FFileHandleWrapper>(TUniquePtr<IFileHandle>(PlatformFile.OpenWrite(*DownloadFile)));
	if (StreamToFileHandle == nullptr || StreamToFileHandle->FileHandle == nullptr)
	{
		FAB_LOG_ERROR("Failed to open file for writing: %s", *DownloadFile);
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		StreamToFileHandle.Reset();

		return;
	}

	DownloadRequest->SetResponseBodyReceiveStreamDelegateV2(FHttpRequestStreamDelegateV2::CreateLambda(
	[this, StreamToFileHandle](const void* DataPtr, int64& DataLength)
	{
		if (StreamToFileHandle != nullptr && StreamToFileHandle->FileHandle != nullptr && DataLength > 0)
		{
			if (!StreamToFileHandle->FileHandle->Write(static_cast<const uint8*>(DataPtr), DataLength))
			{
				// signals error to the HTTP module by setting the data length zero
				DataLength = 0;
				FAB_LOG_ERROR("Failed to write data to file");
				StreamToFileHandle->FileHandle.Reset();
			}
		}
	}));

#if (ENGINE_MAJOR_VERSION >=5 && ENGINE_MINOR_VERSION <=3)
	DownloadRequest->OnRequestProgress().BindLambda(
		[this](FHttpRequestPtr Request, uint32 UploadedBytes, uint32 DownloadedBytes)
#else
	DownloadRequest->OnRequestProgress64().BindLambda(
		[this](FHttpRequestPtr Request, uint64 UploadedBytes, uint64 DownloadedBytes)
#endif
		{
			DownloadStats.CompletedBytes  = DownloadedBytes;
			DownloadStats.PercentComplete = DownloadStats.TotalBytes > 0
				? 100.f * static_cast<float>(DownloadStats.CompletedBytes) / static_cast<float>(DownloadStats.TotalBytes)
				: 0.f;
			// TODO: Calculate Download Speed
			OnDownloadProgressDelegate.Broadcast(this, DownloadStats);
		}
	);
	DownloadRequest->OnProcessRequestComplete().BindLambda(
	[this, SaveFilename, DownloadFile, StreamToFileHandle]
	(FHttpRequestPtr Request, FHttpResponsePtr Response, const bool bRequestComplete)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		bool bFileInDisk = false;
		if (StreamToFileHandle != nullptr && StreamToFileHandle->FileHandle != nullptr)
		{
			bFileInDisk = StreamToFileHandle->FileHandle->Flush(true);
			// Close file handle now:
			StreamToFileHandle->FileHandle.Reset();
		}

		if (DownloadStats.bIsCached)
		{
			PlatformFile.DeleteFile(*DownloadFile);
			OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
			return;
		}

		const int32 ResponseCode = Response.IsValid() ? Response->GetResponseCode() : 0;
		DownloadStats.bIsSuccess = false; // false unless we can download and rename
		if (bFileInDisk && bRequestComplete && ResponseCode >= 200 && ResponseCode < 300)
		{
			// Remove existing file at destination — MoveFile fails if it already exists on Windows
			if (PlatformFile.FileExists(*SaveFilename))
			{
				PlatformFile.DeleteFile(*SaveFilename);
			}

			// Download OK: let's rename the download to the actual final filename
			if (PlatformFile.MoveFile(*SaveFilename, *DownloadFile))
			{
				DownloadStats.DownloadCompletedAt = FDateTime::Now().ToUnixTimestamp();
				DownloadStats.DownloadedFiles = { SaveFilename };
				DownloadStats.bIsSuccess = true;
			}
			else
			{
				FAB_LOG_ERROR("Failed to rename downloaded file from '%s' to '%s'", *DownloadFile, *SaveFilename);
			}
		}

		if (!DownloadStats.bIsSuccess)
		{
			PlatformFile.DeleteFile(*DownloadFile);
		}
	
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
	});

	DownloadStats.DownloadStartedAt = FDateTime::Now().ToUnixTimestamp();
	DownloadRequest->ProcessRequest();
}

void FFabDownloadRequest::ExecuteBuildPatchRequest()
{
	DownloadStats.DownloadStartedAt = FDateTime::Now().ToUnixTimestamp();
	if (!LoadBuildPatchServices())
	{
		FAB_LOG_ERROR("Failed to load BuildPatchServicesModule");
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	FString ManifestURL, BaseURL;
	DownloadURL.Split(",", &ManifestURL, &BaseURL, ESearchCase::CaseSensitive);
	FHttpModule& HTTPModule = FHttpModule::Get();

	DownloadRequest = HTTPModule.CreateRequest().ToSharedPtr();
	DownloadRequest->SetURL(ManifestURL);
	DownloadRequest->OnProcessRequestComplete().BindLambda(
		[this, BaseURL](FHttpRequestPtr Request, FHttpResponsePtr Response, const bool bRequestComplete)
		{
			if (bRequestComplete && Response.IsValid() && Response->GetResponseCode() >= 200 && Response->GetResponseCode() < 300)
			{
				ManifestData = Response->GetContent();
				OnPackManifestDownloaded(BaseURL);
			}
			else
			{
				DownloadStats.bIsSuccess = false;
				OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
			}
		}
	);

	DownloadRequest->ProcessRequest();
}

void FFabDownloadRequest::ExecuteBuildPatchInstallRequest()
{
	DownloadStats.DownloadStartedAt = FDateTime::Now().ToUnixTimestamp();
	if (!LoadBuildPatchServices())
	{
		FAB_LOG_ERROR("Failed to load BuildPatchServicesModule");
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	FString ManifestURL, BaseURL;
	DownloadURL.Split(",", &ManifestURL, &BaseURL, ESearchCase::CaseSensitive);
	FHttpModule& HTTPModule = FHttpModule::Get();

	DownloadRequest = HTTPModule.CreateRequest().ToSharedPtr();
	DownloadRequest->SetURL(ManifestURL);
	DownloadRequest->OnProcessRequestComplete().BindLambda(
		[this, BaseURL](FHttpRequestPtr Request, FHttpResponsePtr Response, const bool bRequestComplete)
		{
			if (bRequestComplete && Response.IsValid() && Response->GetResponseCode() >= 200 && Response->GetResponseCode() < 300)
			{
				ManifestData = Response->GetContent();
				OnPluginManifestDownloaded(BaseURL);
			}
			else
			{
				DownloadStats.bIsSuccess = false;
				OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
			}
		}
	);

	DownloadRequest->ProcessRequest();
}

void FFabDownloadRequest::OnPackManifestDownloaded(const FString& BaseURL)
{
	if (bPendingCancel)
	{
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	BuildPatchServices::FBuildInstallerConfiguration BuildInstallerConfiguration({});
	BuildInstallerConfiguration.InstallDirectory = DownloadLocation;
	BuildInstallerConfiguration.StagingDirectory = FFabAssetsCache::GetCacheLocation() / AssetID;
	BuildInstallerConfiguration.InstallMode      = BuildPatchServices::EInstallMode::NonDestructiveInstall;
	BaseURL.ParseIntoArray(BuildInstallerConfiguration.CloudDirectories, TEXT(","));

	auto Manifest = BuildPatchServices->MakeManifestFromData(ManifestData);
	if (!Manifest.IsValid())
	{
		FAB_LOG_ERROR("Invalid Manifest");
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	DownloadStats.DownloadedFiles = Manifest.GetBuildFileList();
	if (DownloadStats.DownloadedFiles.ContainsByPredicate(
		[](const FString& File)
		{
			const FString Ext = FPaths::GetExtension(File);
			// Also check for truncated extensions due to BPS off-by-one bug
			return Ext == "uproject" || Ext == "uprojec" || Ext == "uplugin" || Ext == "uplugi";
		}
	))
	{
		FAB_LOG_ERROR("Invalid pack - either contains a uproject or a uplugin file");
		DownloadStats.bIsSuccess = false;
		DownloadStats.DownloadedFiles.Empty();
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	auto OnComplete = FBuildPatchInstallerDelegate::CreateLambda(
		[this](const IBuildInstallerRef& Installer)
		{
			DownloadStats.DownloadCompletedAt = FDateTime::Now().ToUnixTimestamp();
			DownloadStats.PercentComplete     = 100.0f;
			DownloadStats.bIsSuccess          = !Installer->IsCanceled();
			if (BpsProgressTickerHandle.IsValid())
			{
				FTSTicker::RemoveTicker(BpsProgressTickerHandle);
			}
			OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		}
	);
	BpsInstaller = BuildPatchServices->CreateInstaller(Manifest, MoveTemp(BuildInstallerConfiguration), MoveTemp(OnComplete)).ToSharedPtr();

	auto OnProgress = FTickerDelegate::CreateLambda(
		[this, Installer = BpsInstaller.ToSharedRef()](const float Delta)
		{
			const int64 TotalDownloaded = BuildPatchServices->GetTotalDownloaded(Installer);
			// const float UpdateProgress        = BuildPatchServices->GetUpdateProgress(Installer);
			const int64 TotalDownloadRequired = BuildPatchServices->GetTotalDownloadRequired(Installer);

			if (TotalDownloadRequired == 0)
			{
				return true;
			}

			DownloadStats.CompletedBytes      = TotalDownloaded;
			DownloadStats.TotalBytes          = TotalDownloadRequired;
			DownloadStats.PercentComplete     = (static_cast<float>(DownloadStats.CompletedBytes) / DownloadStats.TotalBytes) * 100.0f;


			OnDownloadProgressDelegate.Broadcast(this, DownloadStats);
			return true;
		}
	);
	BpsProgressTickerHandle = FTSTicker::GetCoreTicker().AddTicker(MoveTemp(OnProgress), 1.0f);

	BpsInstaller->StartInstallation();
}

void FFabDownloadRequest::OnPluginManifestDownloaded(const FString& BaseURL)
{
	if (bPendingCancel)
	{
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	BuildPatchServices::FBuildInstallerConfiguration BuildInstallerConfiguration({});
	BuildInstallerConfiguration.InstallDirectory = DownloadLocation;
	BuildInstallerConfiguration.StagingDirectory = FFabAssetsCache::GetCacheLocation() / AssetID;
	BuildInstallerConfiguration.InstallMode      = BuildPatchServices::EInstallMode::NonDestructiveInstall;
	BaseURL.ParseIntoArray(BuildInstallerConfiguration.CloudDirectories, TEXT(","));

	auto Manifest = BuildPatchServices->MakeManifestFromData(ManifestData);
	if (!Manifest.IsValid())
	{
		FAB_LOG_ERROR("Invalid Manifest");
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	DownloadStats.DownloadedFiles = Manifest.GetBuildFileList();

	if (!DownloadStats.DownloadedFiles.ContainsByPredicate(
		[](const FString& File) {
			const FString Extension = FPaths::GetExtension(File);
			// Also check for "uplugi" due to BPS off-by-one bug truncating filenames
			return Extension == TEXT("uplugin") || Extension == TEXT("uplugi");
		}))
	{
		FAB_LOG_ERROR("Invalid plugin manifest - does not contain a .uplugin file");
		DownloadStats.bIsSuccess = false;
		DownloadStats.DownloadedFiles.Empty();
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}

	auto OnComplete = FBuildPatchInstallerDelegate::CreateLambda(
		[this](const IBuildInstallerRef& Installer)
		{
			DownloadStats.DownloadCompletedAt = FDateTime::Now().ToUnixTimestamp();
			DownloadStats.PercentComplete     = 100.0f;
			DownloadStats.bIsSuccess          = !Installer->IsCanceled();
			if (BpsProgressTickerHandle.IsValid())
			{
				FTSTicker::RemoveTicker(BpsProgressTickerHandle);
			}
			OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		}
	);
	BpsInstaller = BuildPatchServices->CreateInstaller(Manifest, MoveTemp(BuildInstallerConfiguration), MoveTemp(OnComplete)).ToSharedPtr();

	auto OnProgress = FTickerDelegate::CreateLambda(
		[this, Installer = BpsInstaller.ToSharedRef()](const float Delta)
		{
			const int64 TotalDownloaded = BuildPatchServices->GetTotalDownloaded(Installer);
			// const float UpdateProgress        = BuildPatchServices->GetUpdateProgress(Installer);
			const int64 TotalDownloadRequired = BuildPatchServices->GetTotalDownloadRequired(Installer);

			if (TotalDownloadRequired == 0)
			{
				return true;
			}

			DownloadStats.CompletedBytes      = TotalDownloaded;
			DownloadStats.TotalBytes          = TotalDownloadRequired;
			DownloadStats.PercentComplete     = (static_cast<float>(DownloadStats.CompletedBytes) / DownloadStats.TotalBytes) * 100.0f;


			OnDownloadProgressDelegate.Broadcast(this, DownloadStats);
			return true;
		}
	);
	BpsProgressTickerHandle = FTSTicker::GetCoreTicker().AddTicker(MoveTemp(OnProgress), 1.0f);

	BpsInstaller->StartInstallation();
}

FString FFabDownloadRequest::GetFilenameFromURL(const FString& URL)
{
	int32 SlashIndex = -1;
	if (!URL.FindLastChar('/', SlashIndex) || SlashIndex == -1)
	{
		SlashIndex = 0; // Local file without scheme maybe?
	}

	int32 QuestionIndex = -1;
	if (!URL.FindChar('?', QuestionIndex) || QuestionIndex == -1)
	{
		QuestionIndex = URL.Len();
	}
	return URL.Mid(SlashIndex + 1, QuestionIndex - 1 - SlashIndex);
}

void FFabDownloadRequest::StartDownload()
{
	if (bPendingCancel)
	{
		DownloadStats.bIsSuccess = false;
		OnDownloadCompleteDelegate.Broadcast(this, DownloadStats);
		return;
	}
	if (DownloadType == EFabDownloadType::HTTP)
	{
		ExecuteHTTPRequest();
	}
	else if (DownloadType == EFabDownloadType::BuildPatchRequest)
	{
		ExecuteBuildPatchRequest();
	}
	else if (DownloadType == EFabDownloadType::BuildPatchInstallRequest)
	{
		ExecuteBuildPatchInstallRequest();
	}
}

void FFabDownloadRequest::ExecuteRequest()
{
	FFabDownloadQueue::AddDownloadToQueue(this);
}

void FFabDownloadRequest::Cancel()
{
	bool bWasCancelled = false;
	if (DownloadRequest.IsValid() && DownloadRequest->GetStatus() == EHttpRequestStatus::Processing)
	{
		DownloadStats.bIsSuccess = false;
		DownloadStats.DownloadedFiles.Empty();
		DownloadRequest->CancelRequest();
		bWasCancelled = true;
	}
	if (BpsInstaller.IsValid() && !BpsInstaller->IsComplete() && !BpsInstaller->IsCanceled())
	{
		DownloadStats.bIsSuccess = false;
		DownloadStats.DownloadedFiles.Empty();
		BuildPatchServices->CancelInstall(BpsInstaller.ToSharedRef());
		bWasCancelled = true;
	}
	if (!bWasCancelled)
	{
		bPendingCancel = true;
	}
}

int32 FFabDownloadQueue::DownloadQueueLimit = 2;
TSet<FFabDownloadRequest*> FFabDownloadQueue::DownloadQueue;
TQueue<FFabDownloadRequest*> FFabDownloadQueue::WaitingQueue;

void FFabDownloadQueue::AddDownloadToQueue(FFabDownloadRequest* DownloadRequest)
{
	if (DownloadQueue.Num() >= DownloadQueueLimit)
	{
		WaitingQueue.Enqueue(DownloadRequest);
	}
	else
	{
		DownloadQueue.Add(DownloadRequest);
		DownloadRequest->OnDownloadComplete().AddLambda(
			[DownloadRequest](const FFabDownloadRequest* Request, const FFabDownloadStats& Stats)
			{
				DownloadQueue.Remove(DownloadRequest);
				if (FFabDownloadRequest* NewRequest = nullptr; WaitingQueue.Dequeue(NewRequest))
				{
					AddDownloadToQueue(NewRequest);
				}
			}
		);
		DownloadRequest->StartDownload();
	}
}
