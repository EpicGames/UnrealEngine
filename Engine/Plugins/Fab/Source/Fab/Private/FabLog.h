// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogFab, Log, All);

#define FAB_LOG(Format, ...)         UE_LOG(LogFab, Display, TEXT(Format), ##__VA_ARGS__)
#define FAB_LOG_ERROR(Format, ...)   UE_LOG(LogFab, Error,   TEXT(Format), ##__VA_ARGS__)
#define FAB_LOG_VERBOSE(Format, ...) UE_LOG(LogFab, Display, TEXT(Format), ##__VA_ARGS__)
