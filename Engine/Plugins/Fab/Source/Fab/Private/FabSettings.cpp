// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabSettings.h"

#include "FabAuthentication.h"
#include "FabBrowser.h"
#include "Misc/Paths.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FabSettings)

UFabSettings::UFabSettings()
{
	#if WITH_EDITOR
	for (TFieldIterator<FProperty> It(GetClass()); It; ++It)
	{
		FProperty* Property = *It;
		if (Property && Property->GetMetaData(TEXT("DevOnly")).ToBool())
		{
			Property->SetMetaData(TEXT("Category"), TEXT("HiddenProperties"));
		}
	}
	#endif
}

void UFabSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UFabSettings, CacheDirectoryPath))
		{
			if (!FPaths::DirectoryExists(CacheDirectoryPath.Path) || FPaths::IsRelative(CacheDirectoryPath.Path))
			{
				CacheDirectoryPath = FDirectoryPath { FPlatformProcess::UserTempDir() / FString("FabLibrary") };
			}
		}
		
		SaveConfig();
		
		if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UFabSettings, Environment))
		{
			FabAuthentication::DeletePersistentAuth();
			FabAuthentication::Init();
			if (Environment != EFabEnvironment::CustomUrl)
			{
				FFabBrowser::OpenURL();
			}
		}
		else if (PropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(UFabSettings, CustomUrl))
		{
			if (Environment == EFabEnvironment::CustomUrl)
			{
				FFabBrowser::OpenURL();
			}
		}
	}
}

FString UFabSettings::GetUrlFromEnvironment() const
{
	switch (Environment)
	{
	default:
		// fall through
	case EFabEnvironment::Prod:
		return TEXT("https://fab.com");
	case EFabEnvironment::Gamedev:
		return TEXT("https://fab.cceb.dev.use1a.on.epicgames.com");
	case EFabEnvironment::Test:
		return TEXT("https://fab.daec.live.use1a.on.epicgames.com");
	case EFabEnvironment::CustomUrl:
		return FString();
	}
}
