// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "UObject/SoftObjectPath.h"

#include "FabSettings.generated.h"

UENUM()
enum class EFabEnvironment : uint8
{
	Prod UMETA(DisplayName = "Prod"),
	Gamedev UMETA(DisplayName = "Gamedev"),
	Test UMETA(DisplayName = "Test"),
	CustomUrl UMETA(DisplayName = "Custom URL"),
};

UENUM()
enum class EFabPreferredFormats : uint8
{
	GLTF UMETA(DisplayName = "gltf / glb"),
	FBX UMETA(DisplayName = "fbx"),
};

UENUM()
enum class EFabExperienceMode : uint8
{
	Standard UMETA(DisplayName = "Standard"),
	Performance UMETA(DisplayName = "Performance"),
};

UENUM()
enum class EFabPreferredQualityTier : uint8
{
	Low UMETA(DisplayName = "low"),
	Medium UMETA(DisplayName = "medium"),
	High UMETA(DisplayName = "high"),
	Raw UMETA(DisplayName = "raw")
};

UCLASS(config=EditorPerProjectUserSettings, hideCategories=HiddenProperties)
class UFabSettings : public UObject
{
	GENERATED_BODY()

public:
	UFabSettings();

	/** Frontend used by the Fab plugin (reopen the tab to see the change) */
	UPROPERTY(config, EditAnywhere, Category=Frontend, meta=(DevOnly=true))
	EFabEnvironment Environment = EFabEnvironment::Prod;

	/** URL used when the [Fab (custom)] frontend is selected */
	UPROPERTY(config, EditAnywhere, Category=Frontend, meta=(DevOnly=true))
	FString CustomUrl;

	/** Custom auth token used when it's non-empty */
	UPROPERTY(config, EditAnywhere, Category=Frontend, meta=(DevOnly=true))
	FString CustomAuthToken;

	/** Experience mode: Standard uses OSR rendering, Performance uses native rendering (Windows only; Mac uses a single OSR tab) */
	UPROPERTY(config, EditAnywhere, Category=General)
	EFabExperienceMode ExperienceMode = EFabExperienceMode::Standard;

	/** Enable chrome debug options - default is false */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bEnableDebugOptions = false;

	/** Path to the local library */
	UPROPERTY(config, EditAnywhere, Category = General)
	FDirectoryPath CacheDirectoryPath { FPlatformProcess::UserTempDir() / FString("FabLibrary") };
	
	/** Cache directory */
	UPROPERTY(config, VisibleAnywhere, Category = General)
	FString CacheDirectorySize;

	/** Preferred default format */
	/* The preferred format will always be selected, if not available, the best available format for the product will be chosen. */
	UPROPERTY(config, VisibleAnywhere, Category = ProductFormats, meta=(DevOnly=true))
	FString ProductFormatsSectionSubText = "";

	/** Preferred default format */
	UPROPERTY(config, EditAnywhere, Category = ProductFormats, meta=(DevOnly=true))
	EFabPreferredFormats PreferredDefaultFormat = EFabPreferredFormats::GLTF;
	
	/** Preferred default quality for MS assets */
	UPROPERTY(config, EditAnywhere, Category = Megascans)
	EFabPreferredQualityTier PreferredQualityTier = EFabPreferredQualityTier::Medium;

	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;

	FString GetUrlFromEnvironment() const;
};
