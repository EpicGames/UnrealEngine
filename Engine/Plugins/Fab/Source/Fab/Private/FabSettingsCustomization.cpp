// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabSettingsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "FabSettings.h"
#include "DetailWidgetRow.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#include "PropertyEditing.h"

#include "Utilities/FabAssetsCache.h"

TSharedRef<IDetailCustomization> FFabSettingsCustomization::MakeInstance()
{
	return MakeShareable(new FFabSettingsCustomization);
}

void FFabSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& GeneralCategory = DetailBuilder.EditCategory("General");

	// General section modifications

	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UFabSettings, CacheDirectorySize));

#if PLATFORM_MAC
	// On Mac, WebKit is always natively rendered, so hide this Windows-only setting
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UFabSettings, ExperienceMode));
#endif

	// Hide the default ExperienceMode row so we can add it with custom description text
	DetailBuilder.HideProperty(GET_MEMBER_NAME_CHECKED(UFabSettings, ExperienceMode));

	TArray<TSharedRef<IPropertyHandle>> GeneralProperties;
	GeneralCategory.GetDefaultProperties(GeneralProperties);
	for (const TSharedRef<IPropertyHandle>& PropertyHandle : GeneralProperties)
	{
		// Skip properties we customize manually
		if (PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UFabSettings, CacheDirectorySize)
			|| PropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UFabSettings, ExperienceMode))
		{
			continue;
		}

		// Add the property
		GeneralCategory.AddProperty(PropertyHandle);
	}

#if !PLATFORM_MAC
	{
		const TSharedRef<IPropertyHandle> ExperienceModeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFabSettings, ExperienceMode));

		GeneralCategory.AddCustomRow(FText::FromString("Experience mode"))
		.NameContent()
		[
			ExperienceModeHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 8.f, 0.f, 0.f)
			[
				ExperienceModeHandle->CreatePropertyValueWidget()
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 8.f, 0.f, 8.f)
			[
				SNew(STextBlock)
				.Text(FText::FromString(
					TEXT("Standard (default): Balanced plugin behavior. Supports docked and floating windows, with up to 4 open tabs.\n\n")
					TEXT("Performance: Optimized for handling a higher number of simultaneous tabs. Available in floating window mode only.")))
				.AutoWrapText(true)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			]
		];
	}
#endif

	const TSharedRef<IPropertyHandle> CacheSizeStringHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UFabSettings, CacheDirectorySize));
	GeneralCategory.AddCustomRow(CacheSizeStringHandle->GetPropertyDisplayName())
	.NameContent()
	[
		CacheSizeStringHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SBox)
		.MinDesiredWidth(1400.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SOverlay)
			.FlowDirectionPreference(EFlowDirectionPreference::LeftToRight)

			+ SOverlay::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SEditableTextBox)
				.Text_Static(&FFabAssetsCache::GetCacheSizeString)
				.IsReadOnly(true)
			]
			
			+ SOverlay::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SButton).Text(FText::FromString("Clean Directory"))
				.OnClicked(FOnClicked::CreateRaw(this, &FFabSettingsCustomization::OnButtonClick))
			]
		]
	];
}

FReply FFabSettingsCustomization::OnButtonClick()
{
	FFabAssetsCache::ClearCache();
	return FReply::Handled();
}
