// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class FFabSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	FReply OnButtonClick();
};
