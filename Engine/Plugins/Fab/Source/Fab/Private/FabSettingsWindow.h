// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FabSettings.h"
#include "DetailCategoryBuilder.h"
#include "Widgets/Layout/SBox.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

TObjectPtr<UFabSettings> FabPluginSettings = nullptr;

struct SFabSettingsWindow : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SFabSettingsWindow)
		{
		}

		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		Window = InArgs._WidgetWindow;

		TSharedPtr<SBox> DetailsViewBox;
		ChildSlot[SNew(SVerticalBox) + SVerticalBox::Slot().AutoHeight().Padding(2)[SAssignNew(DetailsViewBox, SBox).MaxDesiredHeight(550.0f).MinDesiredWidth(550.0f)]];

		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
		DetailsViewBox->SetContent(DetailsView);

		if (FabPluginSettings == nullptr)
		{
			FabPluginSettings = GetMutableDefault<UFabSettings>();
		}

		DetailsView->SetObject(FabPluginSettings, true);
	}

	virtual bool SupportsKeyboardFocus() const override { return true; }

	TWeakPtr<SWindow> Window;
};
