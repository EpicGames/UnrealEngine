// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabWorkflowFactoryRegistry.h"

TMap<FString, TSharedPtr<IFabWorkflowFactory>> FFabWorkflowFactoryRegistry::Factories{};

bool FFabWorkflowFactoryRegistry::RegisterFactory(const TSharedPtr<IFabWorkflowFactory>& InFactory)
{
	const TArray<FString>& ImportAssetTypes = InFactory->GetImportAssetTypes();
	for (const FString& ImportAssetType : ImportAssetTypes)
	{
		if (Factories.Contains(ImportAssetType))
		{
			return false;
		}
	}
	for (const FString& ImportAssetType : ImportAssetTypes)
	{
		Factories.Add(ImportAssetType, InFactory);
	}
	return true;
}

void FFabWorkflowFactoryRegistry::UnregisterFactory(const TSharedPtr<IFabWorkflowFactory>& InFactory)
{
	for (const FString& ImportAssetType : InFactory->GetImportAssetTypes())
	{
		Factories.Remove(ImportAssetType);
	}
}

const TSharedPtr<IFabWorkflowFactory>& FFabWorkflowFactoryRegistry::GetFactory(const FString& ImportType)
{
	if (TSharedPtr<IFabWorkflowFactory>* Factory = Factories.Find(ImportType))
	{
		if ((*Factory)->CanImportAssetType(ImportType))
		{
			return *Factory;
		}
	}

	static TSharedPtr<IFabWorkflowFactory> NullFactory{nullptr};
	return NullFactory;
}

bool FFabWorkflowFactoryRegistry::IsAssetTypeRegistered(const FString& AssetType)
{
	return Factories.Contains(AssetType);
}
