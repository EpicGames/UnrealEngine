// Copyright Epic Games, Inc. All Rights Reserved.

#include "Importers/ActorSpawner.h"

#include "Materials/MaterialParameters.h"
#include "StaticMeshResources.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/SkeletalMeshActor.h"

#include "Components/DecalComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"

#include "Engine/DecalActor.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceConstant.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorSpawner)

UFabPlaceholderSpawner::UFabPlaceholderSpawner(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	DisplayName            = FText::FromString("Fab Placeholder Factory");
	NewActorClass          = AActor::StaticClass();
	bUseSurfaceOrientation = true;
}

UFabStaticMeshPlaceholderSpawner::UFabStaticMeshPlaceholderSpawner(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	DisplayName            = FText::FromString("Fab Static Mesh Placeholder Factory");
	NewActorClass          = AStaticMeshActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UFabStaticMeshPlaceholderSpawner::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UStaticMesh::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "InvalidFabStaticMesh", "A Static Mesh should be supplied.");
		return false;
	}
	return true;
}

void UFabStaticMeshPlaceholderSpawner::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	const AStaticMeshActor* StaticMeshActor   = Cast<AStaticMeshActor>(NewActor);
	UStaticMeshComponent* StaticMeshComponent = StaticMeshActor->GetStaticMeshComponent();
	UStaticMesh* StaticMesh                   = Cast<UStaticMesh>(Asset);

	StaticMeshComponent->UnregisterComponent();
	StaticMeshComponent->SetStaticMesh(StaticMesh);
	if (StaticMesh->GetRenderData())
		StaticMeshComponent->StaticMeshDerivedDataKey = StaticMesh->GetRenderData()->DerivedDataKey;
	StaticMeshComponent->RegisterComponent();
	this->OnActorSpawn().ExecuteIfBound(NewActor);
}

UObject* UFabStaticMeshPlaceholderSpawner::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	const AStaticMeshActor* StaticMeshActor = CastChecked<AStaticMeshActor>(Instance);
	check(StaticMeshActor->GetStaticMeshComponent());
	return StaticMeshActor->GetStaticMeshComponent()->GetStaticMesh();
}

UFabSkeletalMeshPlaceholderSpawner::UFabSkeletalMeshPlaceholderSpawner(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	DisplayName            = FText::FromString("Fab Skeletal Mesh Placeholder Factory");
	NewActorClass          = ASkeletalMeshActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UFabSkeletalMeshPlaceholderSpawner::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(USkeletalMesh::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "InvalidFabSkeletalMesh", "A Skeletal Mesh should be supplied.");
		return false;
	}
	return true;
}

void UFabSkeletalMeshPlaceholderSpawner::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);

	ASkeletalMeshActor* SkeletalMeshActor         = Cast<ASkeletalMeshActor>(NewActor);
	USkeletalMeshComponent* SkeletalMeshComponent = SkeletalMeshActor->GetSkeletalMeshComponent();
	USkeletalMesh* SkeletalMesh                   = Cast<USkeletalMesh>(Asset);

	SkeletalMeshComponent->UnregisterComponent();
	SkeletalMeshComponent->SetSkeletalMesh(SkeletalMesh);
	if (SkeletalMeshActor->GetWorld()->IsGameWorld())
		SkeletalMeshActor->ReplicatedMesh = SkeletalMesh;
	SkeletalMeshComponent->RegisterComponent();

	this->OnActorSpawn().ExecuteIfBound(NewActor);
}

UObject* UFabSkeletalMeshPlaceholderSpawner::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	const ASkeletalMeshActor* SkeletalMeshActor = CastChecked<ASkeletalMeshActor>(Instance);
	check(SkeletalMeshActor->GetSkeletalMeshComponent());
	return SkeletalMeshActor->GetSkeletalMeshComponent()->GetSkeletalMeshAsset();
}

UFabDecalPlaceholderSpawner::UFabDecalPlaceholderSpawner(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
	DisplayName            = FText::FromString("Fab Decal Placeholder Factory");
	NewActorClass          = ADecalActor::StaticClass();
	bUseSurfaceOrientation = true;
}

bool UFabDecalPlaceholderSpawner::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (!AssetData.IsValid() || !AssetData.IsInstanceOf(UMaterialInstanceConstant::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "InvalidFabMaterialInstance", "A Material Instance Constant should be supplied.");
		return false;
	}
	if (FString TagValue; AssetData.GetTagValue(GET_MEMBER_NAME_CHECKED(UMaterialInstanceConstant, Parent), TagValue))
	{
		if (!TagValue.Contains("M_MS_Decal", ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			OutErrorMsg = NSLOCTEXT("CanCreateActor", "InvalidFabDecalMaterialInstance", "A Fab Deferred Decal Material Instance Constant should be supplied.");
			return false;
		}
	}
	return true;
}

void UFabDecalPlaceholderSpawner::PostSpawnActor(UObject* Asset, AActor* NewActor)
{
	Super::PostSpawnActor(Asset, NewActor);
	if (!Asset->IsA<UMaterialInstanceConstant>())
	{
		return;
	}

	if (const ADecalActor* DecalActor = Cast<ADecalActor>(NewActor))
	{
		UDecalComponent* DecalComponent  = DecalActor->GetDecal();
		UMaterialInstanceConstant* Decal = Cast<UMaterialInstanceConstant>(Asset);

		DecalComponent->UnregisterComponent();
		DecalComponent->SetDecalMaterial(Decal);
		DecalComponent->RegisterComponent();
		this->OnActorSpawn().ExecuteIfBound(NewActor);
	}
}

UObject* UFabDecalPlaceholderSpawner::GetAssetFromActorInstance(AActor* Instance)
{
	check(Instance->IsA(NewActorClass));
	const ADecalActor* DecalActor = CastChecked<ADecalActor>(Instance);
	check(DecalActor->GetDecal());
	return DecalActor->GetDecal()->GetDecalMaterial();
}
