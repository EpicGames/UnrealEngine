// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/IBuildInstaller.h"
#include "Interfaces/IBuildManifest.h"
#include "Interfaces/IBuildPatchServicesModule.h"

#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"

#if PLATFORM_MAC
#include "Mac/MacPlatformMiscEx.h"
#ifndef RTLD_DEEPBIND
#define RTLD_DEEPBIND 0
#endif
#endif

#if PLATFORM_LINUX
#include <dlfcn.h>
#ifndef RTLD_DEEPBIND
#define RTLD_DEEPBIND 0
#endif
#endif

#if PLATFORM_WINDOWS == 0 && PLATFORM_LINUX == 0 && PLATFORM_MAC == 0
#error "not supported platform"
#endif

namespace BpiLib
{
	namespace BpiLibHelpers
	{
		class FBpiLibHelper;
	}

	class FManifestStorage
	{
	public:
		FManifestStorage(const BpiLibHelpers::FBpiLibHelper& InLibRef, const TArray<uint8>& Data);
		~FManifestStorage();

		IBuildManifestPtr operator->() const { return *ManifestPtr; }
		IBuildManifestPtr* GetManifestPtr() const { return ManifestPtr; }

		bool IsValid() const { return ManifestPtr != nullptr; }

		bool SaveToFile(const FString& Filename) const;

		TArray<FString> GetBuildFileList() const;
		FString GetCustomStringField(const FString& Name) const;
		FString GetAppName() const;

		void SetCustomField(const FString& FieldName, const FString& Value) const;
		void SetCustomField(const FString& FieldName, const double& Value) const;
		void SetCustomField(const FString& FieldName, const int64& Value) const;

	private:
		const BpiLibHelpers::FBpiLibHelper& LibRef;
		IBuildManifestPtr* ManifestPtr;
	};

	class IBpiLib
	{
	public:
		virtual ~IBpiLib() = default;
		virtual bool IsValid() const = 0;
		virtual IBuildInstallerRef CreateInstaller(FManifestStorage& ManifestStorage, const BuildPatchServices::FBuildInstallerConfiguration& Configuration, FBuildPatchInstallerDelegate CompleteDelegate) const = 0;
		virtual void CancelInstall(const IBuildInstallerRef& Installer) const = 0;
		virtual FManifestStorage MakeManifestFromData(const TArray<uint8>& Data) const = 0;
		virtual bool Tick(float) = 0;
		virtual FBuildInstallStats GetBuildStats(const IBuildInstallerRef& Installer) const = 0;
		virtual int64 GetTotalDownloaded(const IBuildInstallerRef& Installer) const = 0;
		virtual int64 GetState(const IBuildInstallerRef& Installer) const = 0;
		virtual float GetUpdateProgress(const IBuildInstallerRef& Installer) const = 0;
		virtual double GetDownloadSpeed(const IBuildInstallerRef& Installer) const = 0;
		virtual int64 GetTotalDownloadRequired(const IBuildInstallerRef& Installer) const = 0;
	};

	class FBpiLibHelperFactory
	{
	public:
		static TUniquePtr<IBpiLib> Create(const FString& FilePath);
	};

	struct FBpiBuildInstallerConfiguration
	{
		static const int MaxCloudDirs = 20;

		// TArray<FInstallerAction> InstallerActions;
		const TCHAR* InstallDirectory = nullptr;
		const TCHAR* StagingDirectory = nullptr;
		const TCHAR* BackupDirectory = nullptr;
		// TArray<FString> ChunkDatabaseFiles;
		const TCHAR* CloudDirectories[MaxCloudDirs];
		int32 CloudDirectoriesNum = 0;
		int32 InstallMode = int32(BuildPatchServices::EInstallMode::NonDestructiveInstall);
		int32 VerifyMode = int32(BuildPatchServices::EVerifyMode::ShaVerifyAllFiles);
		int32 DeltaPolicy = int32(BuildPatchServices::EDeltaPolicy::Skip);
		bool bRunRequiredPrereqs = true;
		bool bSkipPrereqIfAlreadyRan = true;
		bool bAllowConcurrentExecution = false;
		uint64 DownloadRateLimitBps = 0;
		bool bStageWithRawFilenames = false;
		bool bRejectSymlinks = false;

		static FBpiBuildInstallerConfiguration Create(const BuildPatchServices::FBuildInstallerConfiguration& InCfg)
		{
			FBpiBuildInstallerConfiguration Out;

			Out.InstallDirectory = InCfg.InstallDirectory.IsEmpty() ? nullptr : GetData(InCfg.InstallDirectory);
			Out.StagingDirectory = InCfg.StagingDirectory.IsEmpty() ? nullptr : GetData(InCfg.StagingDirectory);
			Out.BackupDirectory = InCfg.BackupDirectory.IsEmpty() ? nullptr : GetData(InCfg.BackupDirectory);

			ensure(InCfg.CloudDirectories.Num() < FBpiBuildInstallerConfiguration::MaxCloudDirs);
			Out.CloudDirectoriesNum = InCfg.CloudDirectories.Num();
			for (int32 i = 0; i < Out.CloudDirectoriesNum; i++)
			{
				Out.CloudDirectories[i] = GetData(InCfg.CloudDirectories[i]);
			}

			Out.InstallMode = int32(InCfg.InstallMode);
			Out.VerifyMode = int32(InCfg.VerifyMode);
			Out.DeltaPolicy = int32(InCfg.DeltaPolicy);

			Out.bRunRequiredPrereqs = InCfg.bRunRequiredPrereqs;
			// Out.bSkipPrereqIfAlreadyRan = InCfg.bSkipPrereqIfAlreadyRan;
			Out.bAllowConcurrentExecution = InCfg.bAllowConcurrentExecution;
			// Out.DownloadRateLimitBps = InCfg.DownloadRateLimitBps;
			// Out.bStageWithRawFilenames = InCfg.bStageWithRawFilenames;
			// Out.bRejectSymlinks = InCfg.bRejectSymlinks;

			return Out;
		}
	};

	struct FBpiBuildInstallStats
	{
		uint32 NumFilesInBuild = 0;
		uint32 NumFilesOutdated = 0;
		uint32 NumFilesToRemove = 0;
		uint32 NumChunksRequired = 0;
		uint32 ChunksQueuedForDownload = 0;
		uint32 ChunksLocallyAvailable = 0;
		uint32 ChunksInChunkDbs = 0;
		uint32 NumChunksDownloaded = 0;
		uint32 NumChunksRecycled = 0;
		uint32 NumChunksReadFromChunkDbs = 0;
		uint32 NumFailedDownloads = 0;
		uint32 NumBadDownloads = 0;
		uint32 NumAbortedDownloads = 0;
		uint32 NumRecycleFailures = 0;
		uint32 NumDriveStoreChunkLoads = 0;
		uint32 NumDriveStoreLoadFailures = 0;
		uint32 NumChunkDbChunksFailed = 0;
		uint64 TotalDownloadedData = 0;
		uint32 ActiveRequestCountPeak = 0;
		double AverageDownloadSpeed = 0.0;
		double PeakDownloadSpeed = 0.0;
		double FinalDownloadSpeed = 0;
		float TheoreticalDownloadTime = 0.f;
		uint64 TotalReadData = 0;
		double AverageDiskReadSpeed = 0.0;
		double PeakDiskReadSpeed = 0.0;
		uint64 TotalWrittenData = 0;
		double AverageDiskWriteSpeed = 0.0;
		double PeakDiskWriteSpeed = 0.0;
		uint32 NumFilesConstructed = 0;
		float InitializeTime = 0.f;
		float ConstructTime = 0.f;
		float UninstallActionTime = 0.f;
		float MoveFromStageTime = 0.f;
		float FileAttributesTime = 0.f;
		float VerifyTime = 0.f;
		float CleanUpTime = 0.f;
		float PrereqTime = 0.f;
		float ProcessPausedTime = 0.f;
		float ProcessActiveTime = 0.f;
		float ProcessExecuteTime = 0.f;
		bool ProcessSuccess = false;
		uint32 NumInstallRetries = 0;
		int32 FailureType = 0;
		int32* RetryFailureTypes = nullptr;
		int32 RetryFailureTypesNum = 0;
		const TCHAR* ErrorCode = nullptr;
		const TCHAR** RetryErrorCodes = nullptr;
		int32 RetryErrorCodesNum = 0;
		const TCHAR* FailureReasonText = nullptr;
		float FinalProgress = 0.f;
		float OverallRequestSuccessRate = 0.f;
		float ExcellentDownloadHealthTime = 0.f;
		float GoodDownloadHealthTime = 0.f;
		float OkDownloadHealthTime = 0.f;
		float PoorDownloadHealthTime = 0.f;
		float DisconnectedDownloadHealthTime = 0.f;
		uint64 ProcessRequiredDiskSpace = 0;
		uint64 ProcessAvailableDiskSpace = 0;
		uint32 DriveStorePeakBytes = 0;
		uint32 NumDriveStoreLostChunks = 0;
		uint64 MemoryStoreSizePeakBytes = 0; 
		uint64 MemoryStoreSizeLimitBytes = 0;
	};

	namespace BpiLibHelpers
	{
		template<typename FuncType>
		FuncType ImportFunction(const FString& Name, void* DllHandle)
		{
			if (DllHandle != nullptr)
			{
				if (auto* Func = (FuncType)FPlatformProcess::GetDllExport(DllHandle, *Name))
				{
					return Func;
				}
			}
			return nullptr;
		}

		template<typename TTraits>
		class TExportedFunc
		{
		public:
			TExportedFunc(void* DllHandle)
				: Ptr(ImportFunction<typename TTraits::FFuncType>(TTraits::GetName(), DllHandle))
			{
				ensure(Ptr != nullptr);
			}

			bool IsValid() const { return Ptr != nullptr; }

			template<typename ...ArgsType>
			auto operator()(ArgsType&& ...Args) const
			{
				return (*Ptr)(Forward<ArgsType>(Args)...);
			}

			typename TTraits::FFuncType Ptr = nullptr;
		};

		namespace FuncTraits
		{
			struct FInit
			{
				using FFuncType = int (*)(const TCHAR*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?Init@Helpers@@YAHPEB_W@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers4InitEPKDs");
#endif
				}
			};

			struct FShutdown
			{
				using FFuncType = void (*)();
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?Shutdown@Helpers@@YAXXZ");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers8ShutdownEv");
#endif
				}
			};

			struct FTick
			{
				using FFuncType = bool (*)(float);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?Tick@Helpers@@YA_NM@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers4TickEf");
#endif
				}
			};

			struct FFree
			{
				using FFuncType = void (*)(void*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?Free@Helpers@@YAXPEAX@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers4FreeEPv");
#endif
				}
			};

			struct FFreeArray
			{
				using FFuncType = void (*)(void**, int32);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?FreeArray@Helpers@@YAXPEAPEAXH@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers9FreeArrayEPPvi");
#endif
				}
			};

			struct FCreateMakeInstall
			{
				using FFuncType = IBuildInstallerRef(*)(
					IBuildManifestPtr*,
					const FBpiBuildInstallerConfiguration*,
					const void*,
					void(*)(const IBuildInstallerRef&, const void*));
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?CreateMakeInstall@Helpers@@YA?AV?$TSharedRef@VIBuildInstaller@@$00@@PEAV?$TSharedPtr@VIBuildManifest@@$00@@PEBUFBpiBuildInstallerConfiguration@1@PEBXP6AXAEBV2@2@Z@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers17CreateMakeInstallEP10TSharedPtrI14IBuildManifestL7ESPMode1EEPKNS_31FBpiBuildInstallerConfigurationEPKvPFvRK10TSharedRefI15IBuildInstallerLS2_1EES9_E");
#endif

				}
			};

			struct FCancelInstall
			{
				using FFuncType = void (*)(IBuildInstaller*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?CancelInstall@Helpers@@YAXPEAVIBuildInstaller@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers13CancelInstallEP15IBuildInstaller");
#endif
				}
			};

			struct FMakeManifestFromData
			{
				using FFuncType = IBuildManifestPtr * (*)(const uint8*, int32);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?MakeManifestFromData@Helpers@@YAPEAV?$TSharedPtr@VIBuildManifest@@$00@@PEBEH@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers20MakeManifestFromDataEPKhi");
#endif
				}
			};

			struct FDeleteManifest
			{
				using FFuncType = void (*)(IBuildManifestPtr*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?DeleteManifest@Helpers@@YAXPEAV?$TSharedPtr@VIBuildManifest@@$00@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers14DeleteManifestEP10TSharedPtrI14IBuildManifestL7ESPMode1EE");
#endif
				}
			};

			struct FSaveManifestToFile
			{
				using FFuncType = bool (*)(IBuildManifestPtr*, const TCHAR*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?SaveManifestToFile@Helpers@@YA_NPEAV?$TSharedPtr@VIBuildManifest@@$00@@PEB_W@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers18SaveManifestToFileEP10TSharedPtrI14IBuildManifestL7ESPMode1EEPKDs");
#endif
				}
			};

			struct FGetBuildFileList
			{
				using FFuncType = TCHAR * *(*)(IBuildManifestPtr*, int32*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetBuildFileList@Helpers@@YAPEAPEA_WPEAV?$TSharedPtr@VIBuildManifest@@$00@@PEAH@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers16GetBuildFileListEP10TSharedPtrI14IBuildManifestL7ESPMode1EEPi");
#endif
				}
			};

			struct FGetCustomStringField
			{
				using FFuncType = TCHAR * (*)(IBuildManifestPtr*, const TCHAR*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetCustomStringField@Helpers@@YAPEA_WPEAV?$TSharedPtr@VIBuildManifest@@$00@@PEB_W@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers20GetCustomStringFieldEP10TSharedPtrI14IBuildManifestL7ESPMode1EEPKDs");
#endif
				}
			};

			struct FSetCustomStringField
			{
				using FFuncType = TCHAR * (*)(IBuildManifestPtr*, const TCHAR*, const TCHAR*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?SetCustomStringField@Helpers@@YAXPEAV?$TSharedPtr@VIBuildManifest@@$00@@PEB_W1@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers20SetCustomStringFieldEP10TSharedPtrI14IBuildManifestL7ESPMode1EEPKDsS6_");
#endif
				}
			};

			struct FSetCustomDoubleField
			{
				using FFuncType = TCHAR * (*)(IBuildManifestPtr*, const TCHAR*, double);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?SetCustomDoubleField@Helpers@@YAXPEAV?$TSharedPtr@VIBuildManifest@@$00@@PEB_WN@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers20SetCustomDoubleFieldEP10TSharedPtrI14IBuildManifestL7ESPMode1EEPKDsd");
#endif
				}
			};

			struct FSetCustomIntField
			{
				using FFuncType = TCHAR * (*)(IBuildManifestPtr*, const TCHAR*, int64);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?SetCustomIntField@Helpers@@YAXPEAV?$TSharedPtr@VIBuildManifest@@$00@@PEB_W_J@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers17SetCustomIntFieldEP10TSharedPtrI14IBuildManifestL7ESPMode1EEPKDsx");
#endif
				}
			};

			struct FGetAppName
			{
				using FFuncType = TCHAR * (*)(IBuildManifestPtr* Manifest);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetAppName@Helpers@@YAPEA_WPEAV?$TSharedPtr@VIBuildManifest@@$00@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers10GetAppNameEP10TSharedPtrI14IBuildManifestL7ESPMode1EE");
#endif
				}
			};

			struct FGetBuildStats
			{
				using FFuncType = FBpiBuildInstallStats * (*)(IBuildInstaller* Installer);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetBuildStats@Helpers@@YAPEAUFBpiBuildInstallStats@1@PEAVIBuildInstaller@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers13GetBuildStatsEP15IBuildInstaller");
#endif
				}
			};

			struct FGetTotalDownloaded
			{
				using FFuncType = int64(*)(IBuildInstaller*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetTotalDownloaded@Helpers@@YA_JPEAVIBuildInstaller@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers18GetTotalDownloadedEP15IBuildInstaller");
#endif
				}
			};

			struct FGetState
			{
				using FFuncType = int64(*)(IBuildInstaller*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetState@Helpers@@YA_JPEAVIBuildInstaller@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers8GetStateEP15IBuildInstaller");
#endif
				}
			};

			struct FGetUpdateProgress
			{
				using FFuncType = float(*)(IBuildInstaller*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetUpdateProgress@Helpers@@YAMPEAVIBuildInstaller@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers17GetUpdateProgressEP15IBuildInstaller");
#endif
				}
			};

			struct FGetDownloadSpeed
			{
				using FFuncType = double(*)(IBuildInstaller*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetDownloadSpeed@Helpers@@YANPEAVIBuildInstaller@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers16GetDownloadSpeedEP15IBuildInstaller");
#endif
				}
			};

			struct FGetTotalDownloadRequired
			{
				using FFuncType = int64(*)(IBuildInstaller*);
				static const TCHAR* GetName()
				{
#if PLATFORM_WINDOWS
					return TEXT("?GetTotalDownloadRequired@Helpers@@YA_JPEAVIBuildInstaller@@@Z");
#elif PLATFORM_LINUX || PLATFORM_MAC
					return TEXT("_ZN7Helpers24GetTotalDownloadRequiredEP15IBuildInstaller");
#endif
				}
			};
		}

		class FBpiLibHelper : public IBpiLib
		{
			struct FCallbackStorage
			{
				FCallbackStorage(FBuildPatchInstallerDelegate&& InCompleteDelegate) :
					CompleteDelegate(MoveTemp(InCompleteDelegate)) {}

				FBuildPatchInstallerDelegate CompleteDelegate;
			};
		public:
			FBpiLibHelper(void* InDllHandle)
				: DllHandle(InDllHandle)
			{
				ensure(DllHandle != nullptr);
				if (FuncFInit.IsValid())
				{
					bIsInited = FuncFInit(FCommandLine::Get()) == 0;
				}
			}

			virtual ~FBpiLibHelper()
			{
				if (FuncFShutdown.IsValid())
				{
					FuncFShutdown();
				}
				FPlatformProcess::FreeDllHandle(DllHandle);
			}

			virtual bool IsValid() const override { return bIsInited; }

			virtual IBuildInstallerRef CreateInstaller(
				FManifestStorage& ManifestStorage,
				const BuildPatchServices::FBuildInstallerConfiguration& Configuration,
				FBuildPatchInstallerDelegate InCompleteDelegate) const override
			{
				FCallbackStorage* CbStorage = new FCallbackStorage(MoveTemp(InCompleteDelegate));
				FBpiBuildInstallerConfiguration LibCfg = FBpiBuildInstallerConfiguration::Create(Configuration);
				return FuncFCreateMakeInstall(ManifestStorage.GetManifestPtr(),
					&LibCfg,
					CbStorage,
					[](const IBuildInstallerRef& Installer, const void* UserPtr) {
						if (UserPtr)
						{
							const FCallbackStorage* ExtCbStorage = static_cast<const FCallbackStorage*>(UserPtr);
							ExtCbStorage->CompleteDelegate.ExecuteIfBound(Installer);
							delete ExtCbStorage;
						}
					}
				);
			}

			virtual FManifestStorage MakeManifestFromData(const TArray<uint8>& Data) const override
			{
				return FManifestStorage(*this, Data);
			}

			virtual bool Tick(float Delta) override { return FuncFTick(Delta); }

			virtual FBuildInstallStats GetBuildStats(const IBuildInstallerRef& Installer) const override
			{
				FBuildInstallStats Out;
				FBpiBuildInstallStats* InStats = FuncFGetBuildStats(&Installer.Get());

#define COPY(__x__) Out.__x__ = InStats->__x__

				COPY(NumFilesInBuild);
				COPY(NumFilesOutdated);
				COPY(NumFilesToRemove);
				COPY(NumChunksRequired);
				COPY(ChunksQueuedForDownload);
				COPY(ChunksLocallyAvailable);
				COPY(ChunksInChunkDbs);
				COPY(NumChunksDownloaded);
				COPY(NumChunksRecycled);
				COPY(NumChunksReadFromChunkDbs);
				COPY(NumFailedDownloads);
				COPY(NumBadDownloads);
				COPY(NumAbortedDownloads);
				COPY(NumRecycleFailures);
				COPY(NumDriveStoreChunkLoads);
				COPY(NumDriveStoreLoadFailures);
				COPY(NumChunkDbChunksFailed);
				COPY(TotalDownloadedData);
				COPY(ActiveRequestCountPeak);
				COPY(AverageDownloadSpeed);
				COPY(PeakDownloadSpeed);
				COPY(FinalDownloadSpeed);
				COPY(TheoreticalDownloadTime);
				COPY(TotalReadData);
				COPY(AverageDiskReadSpeed);
				COPY(PeakDiskReadSpeed);
				COPY(TotalWrittenData);
				COPY(AverageDiskWriteSpeed);
				COPY(PeakDiskWriteSpeed);
				COPY(NumFilesConstructed);
				COPY(InitializeTime);
				COPY(ConstructTime);
				//COPY(UninstallActionTime); uncomment when merged to ue5
				COPY(MoveFromStageTime);
				COPY(FileAttributesTime);
				COPY(VerifyTime);
				COPY(CleanUpTime);
				COPY(PrereqTime);
				COPY(ProcessPausedTime);
				COPY(ProcessActiveTime);
				COPY(ProcessExecuteTime);
				COPY(ProcessSuccess);
				COPY(NumInstallRetries);

				Out.FailureType = EBuildPatchInstallError(InStats->FailureType);

				// Out.RetryFailureTypes = ConvertAndFree<TArray<EBuildPatchInstallError>>(InStats->RetryFailureTypes, InStats->RetryFailureTypesNum);

				// Out.ErrorCode = ConvertAndFree(InStats->ErrorCode);

				// Out.RetryErrorCodes = ConvertAndFree<TArray<FString>>(InStats->RetryErrorCodes, InStats->RetryErrorCodesNum);
				// Out.FailureReasonText = FText::FromString(ConvertAndFree(InStats->FailureReasonText));

				COPY(FinalProgress);
				COPY(OverallRequestSuccessRate);
				COPY(ExcellentDownloadHealthTime);
				COPY(GoodDownloadHealthTime);
				COPY(OkDownloadHealthTime);
				COPY(PoorDownloadHealthTime);
				COPY(DisconnectedDownloadHealthTime);
				COPY(ProcessRequiredDiskSpace);
				COPY(ProcessAvailableDiskSpace);
				// COPY(DriveStorePeakBytes);
				// COPY(NumDriveStoreLostChunks);
				// COPY(MemoryStoreSizePeakBytes);
				// COPY(MemoryStoreSizeLimitBytes);
#undef COPY
				return Out;
			}

			virtual void CancelInstall(const IBuildInstallerRef& Installer) const override { FuncFCancelInstall(&Installer.Get()); }

			virtual int64 GetTotalDownloaded(const IBuildInstallerRef& Installer) const override { return FuncFGetTotalDownloaded(&Installer.Get()); }
			virtual int64 GetState(const IBuildInstallerRef& Installer) const override { return FuncFGetState(&Installer.Get()); }
			virtual float GetUpdateProgress(const IBuildInstallerRef& Installer) const override { return FuncFGetUpdateProgress(&Installer.Get()); }
			virtual double GetDownloadSpeed(const IBuildInstallerRef& Installer) const override { return FuncFGetDownloadSpeed(&Installer.Get()); }
			virtual int64 GetTotalDownloadRequired(const IBuildInstallerRef& Installer) const override { return FuncFGetTotalDownloadRequired(&Installer.Get()); }

			template<typename Out, typename In>
			Out ConvertAndFree(In Data, int32 Num) const
			{
				Out Result;
				if (Data != nullptr)
				{
					Result.Reserve(Num);
					for (int32 i = 0; i < Num; i++)
					{
						Result.Emplace(typename Out::ElementType(Data[i]));
					}
					FuncFFreeArray((void**)Data, Num);
				}
				return Result;

			}

			FString ConvertAndFree(const TCHAR* Data) const
			{
				FString Result;
				int32 Num = 0;
				if (Data != nullptr)
				{
					Result = Data;
					FuncFFree((void*)Data);
				}
				return Result;
			}

		private:
			bool bIsInited = false;
			void* DllHandle = nullptr;

		public:
			// declare function by a class name
#define DECLFUNC(__x__) TExportedFunc<FuncTraits::__x__> Func##__x__{ DllHandle }

			DECLFUNC(FInit);
			DECLFUNC(FTick);
			DECLFUNC(FShutdown);
			DECLFUNC(FFreeArray);
			DECLFUNC(FFree);

			DECLFUNC(FCreateMakeInstall);
			DECLFUNC(FMakeManifestFromData);
			DECLFUNC(FDeleteManifest);
			DECLFUNC(FSaveManifestToFile);
			DECLFUNC(FGetBuildFileList);
			DECLFUNC(FGetCustomStringField);

			DECLFUNC(FSetCustomStringField);
			DECLFUNC(FSetCustomDoubleField);
			DECLFUNC(FSetCustomIntField);

			DECLFUNC(FGetAppName);
			DECLFUNC(FGetBuildStats);
			DECLFUNC(FGetTotalDownloaded);
			DECLFUNC(FGetState);
			DECLFUNC(FGetUpdateProgress);
			DECLFUNC(FGetDownloadSpeed);
			DECLFUNC(FGetTotalDownloadRequired);

			DECLFUNC(FCancelInstall);

#undef DECLFUNC
		};
	}

	inline TUniquePtr<IBpiLib> FBpiLibHelperFactory::Create(const FString& FilePath)
	{
		if (FPaths::FileExists(FilePath))
		{
#if PLATFORM_WINDOWS
			if (void* DllHandle = FPlatformProcess::GetDllHandle(*FilePath))
#elif PLATFORM_LINUX || PLATFORM_MAC
			FString AbsolutePath = FPaths::ConvertRelativePathToFull(FilePath);
			if (void* DllHandle = dlopen(TCHAR_TO_UTF8(*AbsolutePath), RTLD_NOW | RTLD_DEEPBIND))
#endif
			{
				auto BptLib = MakeUnique<BpiLibHelpers::FBpiLibHelper>(DllHandle);
				return BptLib.IsValid() ? MoveTemp(BptLib) : nullptr;
			}
		}
		return nullptr;
	}

	inline FManifestStorage::FManifestStorage(const BpiLibHelpers::FBpiLibHelper& InLibRef, const TArray<uint8>& Data)
		: LibRef(InLibRef)
	{
		ManifestPtr = LibRef.FuncFMakeManifestFromData(Data.GetData(), Data.Num());
	}

	inline FManifestStorage::~FManifestStorage()
	{
		LibRef.FuncFDeleteManifest(ManifestPtr);
	}

	inline bool FManifestStorage::SaveToFile(const FString& Filename) const
	{
		return LibRef.FuncFSaveManifestToFile(ManifestPtr, *Filename);
	}

	inline TArray<FString> FManifestStorage::GetBuildFileList() const
	{
		int32 Num = 0;
		TCHAR** Data = LibRef.FuncFGetBuildFileList(ManifestPtr, &Num);
		return LibRef.ConvertAndFree<TArray<FString>>(Data, Num);
	}

	inline FString FManifestStorage::GetCustomStringField(const FString& Name) const
	{
		TCHAR* Data = LibRef.FuncFGetCustomStringField(ManifestPtr, *Name);
		return LibRef.ConvertAndFree(Data);
	}

	inline void FManifestStorage::SetCustomField(const FString& FieldName, const FString& Value) const
	{
		LibRef.FuncFSetCustomStringField(ManifestPtr, *FieldName, *Value);
	}

	inline void FManifestStorage::SetCustomField(const FString& FieldName, const double& Value) const
	{
		LibRef.FuncFSetCustomDoubleField(ManifestPtr, *FieldName, Value);
	}

	inline void FManifestStorage::SetCustomField(const FString& FieldName, const int64& Value) const
	{
		LibRef.FuncFSetCustomIntField(ManifestPtr, *FieldName, Value);
	}

	inline FString FManifestStorage::GetAppName() const
	{
		TCHAR* Data = LibRef.FuncFGetAppName(ManifestPtr);
		return LibRef.ConvertAndFree(Data);
	}
}