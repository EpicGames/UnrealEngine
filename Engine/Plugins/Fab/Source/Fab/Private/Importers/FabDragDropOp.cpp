// Copyright Epic Games, Inc. All Rights Reserved.

#include "Importers/FabDragDropOp.h"

#include "Importers/ActorSpawner.h"
#include "ClassIconFinder.h"
#include "Editor.h"

#include "ActorFactories/ActorFactoryBasicShape.h"

#include "Engine/DecalActor.h"
#include "Engine/SkeletalMesh.h"

#include "Materials/Material.h"

#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Layout/SBox.h"

FFabDragDropOp::FFabDragDropOp(const EDragAssetType InDragAssetType)
	: SpawnedActor(nullptr)
	, DragAssetType(InDragAssetType)
{}

FFabDragDropOp::~FFabDragDropOp()
{
	Cancel();
}

TSharedPtr<SWidget> FFabDragDropOp::GetDefaultDecorator() const
{
	const FSlateBrush* Image = nullptr;
	if (DragAssetType == EDragAssetType::Mesh)
	{
		Image = FClassIconFinder::FindThumbnailForClass(UStaticMesh::StaticClass());
	}
	else if (DragAssetType == EDragAssetType::Material)
	{
		Image = FClassIconFinder::FindThumbnailForClass(UMaterial::StaticClass());
	}
	else if (DragAssetType == EDragAssetType::Decal)
	{
		Image = FClassIconFinder::FindThumbnailForClass(ADecalActor::StaticClass());
	}

	return SNew(SBorder)
		[
			SNew(SBox)
			.HeightOverride(80)
			.WidthOverride(80)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SColorBlock)
					.Color(FColor(32, 32, 36).ReinterpretAsLinear())
					.Size(FVector2D(80.0f, 80.0f))
					.UseSRGB(false)
					.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
					.CornerRadius(FVector4(10.0f))
				]
				+ SOverlay::Slot()
				.Padding(10.0f)
				[
					SNew(SImage)
					.Image(Image)
				]
			]
		];
}

void FFabDragDropOp::OnDragged(const FDragDropEvent& DragDropEvent)
{
	if (CursorDecoratorWindow.IsValid())
	{
		CursorDecoratorWindow->MoveWindowTo(DragDropEvent.GetScreenSpacePosition());
	}
}

void FFabDragDropOp::Construct()
{
	MouseCursor = EMouseCursor::GrabHandClosed;
	if (DragAssetType == EDragAssetType::Material)
	{
		EditorApplyHandle = FEditorDelegates::OnApplyObjectToActor.AddLambda(
			[this](const UObject* Object, AActor* Actor)
			{
				if (GetAssets()[0].GetAsset() == Object)
					SpawnedActor = Actor;
			}
		);
	}
	FDragDropOperation::Construct();
}

void FFabDragDropOp::OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent)
{
	if (EditorApplyHandle.IsValid())
	{
		FEditorDelegates::OnApplyObjectToActor.Remove(EditorApplyHandle);
		EditorApplyHandle.Reset();
	}
	DestroyCursorDecoratorWindow();
	if (!bDropWasHandled)
		SpawnedActor = nullptr;
	else if (OnDropDelegate.IsBound())
		OnDropDelegate.Execute();
}

void FFabDragDropOp::DestroyWindow()
{
	DestroyCursorDecoratorWindow();
}

void FFabDragDropOp::DestroySpawnedActor()
{
	if (SpawnedActor.IsValid() && (DragAssetType == EDragAssetType::Mesh || DragAssetType == EDragAssetType::Decal))
	{
		SpawnedActor->Destroy();
	}

	SpawnedActor = nullptr;
}

void FFabDragDropOp::Cancel()
{
	if (EditorApplyHandle.IsValid())
	{
		FEditorDelegates::OnApplyObjectToActor.Remove(EditorApplyHandle);
		EditorApplyHandle.Reset();
	}
	if (UFabPlaceholderSpawner* PlaceholderFactory = Cast<UFabPlaceholderSpawner>(GetActorFactory()))
		PlaceholderFactory->OnActorSpawn().Unbind();
	if (OnDropDelegate.IsBound())
		OnDropDelegate.Unbind();
	DestroyWindow();
}

TSharedPtr<FFabDragDropOp> FFabDragDropOp::New(FAssetData Asset, EDragAssetType InDragAssetType)
{
	TSharedPtr<FFabDragDropOp> Operation = MakeShared<FFabDragDropOp>(InDragAssetType);

	UFabPlaceholderSpawner* ActorFactory = nullptr;
	if (InDragAssetType == EDragAssetType::Mesh)
	{
		const UClass* AssetClass = Asset.GetAsset()->GetClass();
		if (AssetClass->IsChildOf<UStaticMesh>())
		{
			ActorFactory = Cast<UFabPlaceholderSpawner>(GEditor->FindActorFactoryByClass(UFabStaticMeshPlaceholderSpawner::StaticClass()));
		}
		else if (AssetClass->IsChildOf<USkeletalMesh>())
		{
			ActorFactory = Cast<UFabPlaceholderSpawner>(GEditor->FindActorFactoryByClass(UFabSkeletalMeshPlaceholderSpawner::StaticClass()));
		}
	}
	else if (InDragAssetType == EDragAssetType::Decal)
	{
		ActorFactory = Cast<UFabPlaceholderSpawner>(GEditor->FindActorFactoryByClass(UFabDecalPlaceholderSpawner::StaticClass()));
	}

	if (ActorFactory)
	{
		ActorFactory->OnActorSpawn().BindLambda(
			[Operation](AActor* Actor)
			{
				if (Actor->IsActorBeingDestroyed())
					Operation->SpawnedActor = nullptr;
				else
					Operation->SpawnedActor = Actor;
			}
		);
	}

	Operation->Init(
		{
			Asset
		},
		TArray<FString>(),
		ActorFactory
	);
	Operation->Construct();
	return Operation;
}
