// Copyright Epic Games, Inc. All Rights Reserved.

#include "Importers/GenericAssetImporter.h"

#include "AssetImportTask.h"
#include "IAssetTools.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangeGenericMaterialPipeline.h"
#include "InterchangeGenericMeshPipeline.h"
#include "InterchangeGenericTexturePipeline.h"
#include "InterchangeProjectSettings.h"

#include "Runtime/Launch/Resources/Version.h"

#if (ENGINE_MAJOR_VERSION >=5 && ENGINE_MINOR_VERSION <=3)
#include "InterchangeglTFPipeline.h"
#endif

#include "Async/Async.h"

#include "Factories/FbxImportUI.h"
#include "Factories/FbxStaticMeshImportData.h"

UObject* FFabGenericImporter::GetImportOptions(const FString& SourceFile, UObject* const OptionsOuter)
{
	check(OptionsOuter != nullptr);

	const UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
	const UInterchangeSourceData* SourceData      = UInterchangeManager::CreateSourceData(SourceFile);
	if (InterchangeManager.IsInterchangeImportEnabled() && InterchangeManager.CanTranslateSourceData(SourceData))
	{
		const FName PipelineStackName                               = FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(false, *SourceData);
		const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(false);
		if (const FInterchangePipelineStack* const PipelineStack = InterchangeImportSettings.PipelineStacks.Find(PipelineStackName))
		{
			const TArray<FSoftObjectPath>* Pipelines = &PipelineStack->Pipelines;
			UE::Interchange::FScopedTranslator ScopedTranslator(SourceData);
			for (const FInterchangeTranslatorPipelines& TranslatorPipelines : PipelineStack->PerTranslatorPipelines)
			{
				const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
				if (ScopedTranslator.GetTranslator() && ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
				{
					Pipelines = &TranslatorPipelines.Pipelines;
					break;
				}
			}

			UInterchangePipelineStackOverride* StackOverride = NewObject<UInterchangePipelineStackOverride>(OptionsOuter);

			for (const FSoftObjectPath& Pipeline : *Pipelines)
			{
				UInterchangePipelineBase* const DefaultPipeline = Cast<UInterchangePipelineBase>(Pipeline.TryLoad());
				if (!DefaultPipeline)
				{
					continue;
				}
				if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(Pipeline))
				{
					GeneratedPipeline->TransferAdjustSettings(DefaultPipeline);
					GeneratedPipeline->AddToRoot();
					if (UInterchangeGenericAssetsPipeline* const GenericAssetsPipeline = Cast<UInterchangeGenericAssetsPipeline>(GeneratedPipeline))
					{
						GenericAssetsPipeline->MeshPipeline->bImportStaticMeshes                  = true;
						GenericAssetsPipeline->MeshPipeline->bImportSkeletalMeshes                = true;
						GenericAssetsPipeline->MeshPipeline->CombineStaticMeshesBehavior          = EInterchangeCombineStaticMeshesBehavior::All;
						GenericAssetsPipeline->MeshPipeline->CombineSkeletalMeshesBehavior        = EInterchangeCombineSkeletalMeshesBehavior::BySkeleton;
						GenericAssetsPipeline->MeshPipeline->SkeletalMeshImportContentType        = EInterchangeSkeletalMeshContentType::All;
						GenericAssetsPipeline->MeshPipeline->bGenerateLightmapUVs                 = true;
						GenericAssetsPipeline->MeshPipeline->bBuildNanite                         = false;
						GenericAssetsPipeline->MaterialPipeline->bImportMaterials                 = true;
						GenericAssetsPipeline->MaterialPipeline->TexturePipeline->bImportTextures = true;

						GenericAssetsPipeline->MaterialPipeline->MaterialImport                = EInterchangeMaterialImportOption::ImportAsMaterialInstances;
						GenericAssetsPipeline->CommonMeshesProperties->bRecomputeNormals       = false;
						GenericAssetsPipeline->CommonMeshesProperties->bComputeWeightedNormals = false;
						GenericAssetsPipeline->CommonMeshesProperties->VertexColorImportOption = EInterchangeVertexColorImportOption::IVCIO_Replace;
					}
					if (UInterchangeGenericTexturePipeline* const GenericTexturePipeline = Cast<UInterchangeGenericTexturePipeline>(GeneratedPipeline))
					{
						GenericTexturePipeline->bAllowNonPowerOfTwo     = true;
						GenericTexturePipeline->bDetectNormalMapTexture = true;
					}
					#if (ENGINE_MAJOR_VERSION >=5 && ENGINE_MINOR_VERSION <=3)
					{
						if (UInterchangeGLTFPipeline* const GLTFGeneratedPipeline = Cast<UInterchangeGLTFPipeline>(GeneratedPipeline))
						{
							GLTFGeneratedPipeline->bUseGLTFMaterialInstanceLibrary = true;
						}
					}
					#endif

					StackOverride->OverridePipelines.Add(GeneratedPipeline);
				}
			}
			return StackOverride;
		}
		return nullptr;
	}
	if (FPaths::GetExtension(SourceFile).ToLower() == "fbx")
	{
		UFbxImportUI* ImportOptions = NewObject<UFbxImportUI>(OptionsOuter);

		ImportOptions->bIsReimport                                   = false;
		ImportOptions->bImportMesh                                   = true;
		ImportOptions->bImportAnimations                             = true;
		ImportOptions->bImportMaterials                              = true;
		ImportOptions->bImportTextures                               = true;
		ImportOptions->bImportAsSkeletal                             = false;
		ImportOptions->StaticMeshImportData->bCombineMeshes          = true;
		ImportOptions->StaticMeshImportData->bBuildNanite            = false;
		ImportOptions->StaticMeshImportData->bGenerateLightmapUVs    = false;
		ImportOptions->StaticMeshImportData->bAutoGenerateCollision  = false;
		ImportOptions->StaticMeshImportData->VertexColorImportOption = EVertexColorImportOption::Replace;
		ImportOptions->StaticMeshImportData->NormalImportMethod      = FBXNIM_ImportNormalsAndTangents;

		return ImportOptions;
	}

	return nullptr;
}

void FFabGenericImporter::CleanImportOptions(UObject* const Options)
{
	if (const UInterchangePipelineStackOverride* const InterchangeOptions = Cast<UInterchangePipelineStackOverride>(Options))
	{
		for (const FSoftObjectPath& OverridePipeline : InterchangeOptions->OverridePipelines)
		{
			if (UObject* const LoadedPipeline = OverridePipeline.TryLoad())
				LoadedPipeline->RemoveFromRoot();
		}
	}
}

void FFabGenericImporter::ImportAsset(const TArray<FString>& Sources, const FString& Destination, const TFunction<void(const TArray<UObject*>&)>& Callback)
{
	TSharedPtr<TArray<UAssetImportTask*>> MeshImportTasks = MakeShared<TArray<UAssetImportTask*>>();

	for (const FString& Source : Sources)
	{
		UAssetImportTask* MeshImportTask = NewObject<UAssetImportTask>();
		MeshImportTask->AddToRoot();

		MeshImportTask->bAutomated = true;
		MeshImportTask->bSave      = false;
		MeshImportTask->bAsync     = true;
		MeshImportTask->Filename   = Source;

		MeshImportTask->DestinationPath  = Destination;
		MeshImportTask->bReplaceExisting = true;
		MeshImportTask->Options          = GetImportOptions(Source, MeshImportTask);

		MeshImportTasks->Add(MeshImportTask);
	}

	IAssetTools::Get().ImportAssetTasks(*MeshImportTasks);

	TSharedPtr<TArray<UObject*>> ImportedObjects = MakeShared<TArray<UObject*>>();
	Async(
		EAsyncExecution::Thread,
		[MeshImportTasks, ImportedObjects]()
		{
			for (const UAssetImportTask* const MeshImportTask : *MeshImportTasks)
			{
				if (MeshImportTask->AsyncResults.IsValid())
				{
					FPlatformProcess::ConditionalSleep([=]() { return MeshImportTask->IsAsyncImportComplete(); }, 0.25f);
					ImportedObjects->Append(MeshImportTask->AsyncResults->GetImportedObjects());
				}
				else
				{
					ImportedObjects->Append(MeshImportTask->GetObjects());
				}
			}
		},
		[MeshImportTasks, ImportedObjects, Callback]()
		{
			Async(
				EAsyncExecution::TaskGraphMainThread,
				[MeshImportTasks, ImportedObjects, Callback]()
				{
					Callback(*ImportedObjects);
					for (UAssetImportTask* MeshImportTask : *MeshImportTasks)
					{
						CleanImportOptions(MeshImportTask->Options);
						MeshImportTask->RemoveFromRoot();
					}
				}
			);
		}
	);
}
