// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuixelGLTFImporter.h"

#include "Editor.h"
#include "InterchangeGenericAssetsPipeline.h"
#include "InterchangeGenericMaterialPipeline.h"
#include "InterchangeGenericMeshPipeline.h"
#include "InterchangeManager.h"
#include "Runtime/Launch/Resources/Version.h"

#if (ENGINE_MAJOR_VERSION >=5 && ENGINE_MINOR_VERSION <=3)
#include "InterchangeglTFPipeline.h"
#endif

#include "InterchangeGenericTexturePipeline.h"
#include "Materials/MaterialParameters.h"

#include "AssetRegistry/AssetRegistryModule.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"

#include "Engine/Texture.h"
#include "Engine/Texture2D.h"

#include "Kismet2/KismetEditorUtilities.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"

#include "Misc/FileHelper.h"

#include "Pipelines/InterchangeMegascansPipeline.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"

#include "UObject/SoftObjectPath.h"

void FQuixelGltfImporter::SetupGlobalFoliageActor(const FString& ImportPath)
{
	const FString GlobalFoliageActorPackageName     = "BP_GlobalFoliageActor";
	const FString GlobalFoliageActorDestinationPath = FPaths::GetPath(FPaths::GetPath(ImportPath)) / GlobalFoliageActorPackageName;

	if (IAssetRegistry::Get()->DoesPackageExistOnDisk(*GlobalFoliageActorDestinationPath) || FindPackage(nullptr, *GlobalFoliageActorDestinationPath))
	{
		return;
	}

	const FString GlobalFoliageActorClass     = "BP_GlobalFoliageActor_UE5.BP_GlobalFoliageActor_UE5_C";
	const FString GlobalFoliageActorClassPath = "/Fab/Actors/GlobalFoliageActor" / GlobalFoliageActorClass;

	if (UPackage* Package = CreatePackage(*GlobalFoliageActorDestinationPath))
	{
		UClass* ParentClass   = Cast<UClass>(FSoftObjectPath(GlobalFoliageActorClassPath).TryLoad());
		UBlueprint* Blueprint = FKismetEditorUtilities::CreateBlueprint(
			ParentClass,
			Package,
			*GlobalFoliageActorPackageName,
			BPTYPE_Const,
			UBlueprint::StaticClass(),
			UBlueprintGeneratedClass::StaticClass()
		);

		if (Blueprint)
		{
			FAssetRegistryModule::AssetCreated(Blueprint);
			Package->MarkPackageDirty();
		}
	}
}

TArray<FSoftObjectPath> FQuixelGltfImporter::GetPipelinesForSourceData(const UInterchangeSourceData* InSourceData)
{
	TArray<FSoftObjectPath> ImportPipelines;
	ImportPipelines.Add(FSoftObjectPath("/Interchange/Pipelines/DefaultGLTFAssetsPipeline.DefaultGLTFAssetsPipeline"));
	ImportPipelines.Add(FSoftObjectPath("/Interchange/Pipelines/DefaultGLTFPipeline.DefaultGLTFPipeline"));
	return ImportPipelines;
}

void FQuixelGltfImporter::GeneratePipelines(const TArray<FSoftObjectPath>& OriginalPipelines, TArray<UInterchangePipelineBase*>& GeneratedPipelines)
{
	if (!GeneratedPipelines.IsEmpty())
		GeneratedPipelines.Empty();

	for (const FSoftObjectPath& Pipeline : OriginalPipelines)
	{
		UInterchangePipelineBase* const DefaultPipeline = Cast<UInterchangePipelineBase>(Pipeline.TryLoad());
		if (!DefaultPipeline)
		{
			continue;
		}
		if (UInterchangePipelineBase* const GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(Pipeline))
		{
			GeneratedPipeline->TransferAdjustSettings(DefaultPipeline);
			GeneratedPipeline->AddToRoot();
			GeneratedPipelines.Add(GeneratedPipeline);
			#if (ENGINE_MAJOR_VERSION >=5 && ENGINE_MINOR_VERSION <=3)
			{
				if (UInterchangeGLTFPipeline* const GLTFGeneratedPipeline = Cast<UInterchangeGLTFPipeline>(GeneratedPipeline))
				{
					GLTFGeneratedPipeline->bUseGLTFMaterialInstanceLibrary = true;
				}
			}
			#endif
		}
	}

	UInterchangeMegascansPipeline* MegascansPipeline = NewObject<UInterchangeMegascansPipeline>();
	MegascansPipeline->AddToRoot();
	GeneratedPipelines.Add(MegascansPipeline);

	UInterchangeGenericAssetsPipeline* AssetPipeline = GetGenericAssetPipeline(GeneratedPipelines);
	if (AssetPipeline)
	{
		MegascansPipeline->SetCommonMeshesProperties(AssetPipeline->MeshPipeline->CommonMeshesProperties);
	}
}

UInterchangeGenericAssetsPipeline* FQuixelGltfImporter::GetGenericAssetPipeline(const TArray<UInterchangePipelineBase*>& GeneratedPipelines)
{
	if (UInterchangePipelineBase* const* const AssetPipeline = GeneratedPipelines.FindByPredicate(
		[](const UInterchangePipelineBase* Pipeline) { return Pipeline->IsA<UInterchangeGenericAssetsPipeline>(); }
	))
	{
		return Cast<UInterchangeGenericAssetsPipeline>(*AssetPipeline);
	}
	return nullptr;
}

UInterchangeMegascansPipeline* FQuixelGltfImporter::GetMegascanPipeline(const TArray<UInterchangePipelineBase*>& GeneratedPipelines)
{
	if (UInterchangePipelineBase* const* const AssetPipeline = GeneratedPipelines.FindByPredicate(
		[](const UInterchangePipelineBase* Pipeline) { return Pipeline->IsA<UInterchangeMegascansPipeline>(); }
	))
	{
		return Cast<UInterchangeMegascansPipeline>(*AssetPipeline);
	}
	return nullptr;
}

void FQuixelGltfImporter::ImportGltfDecalAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone)
{
	const UInterchangeSourceData* InSourceData = UInterchangeManager::CreateSourceData(SourcePath);

	TArray<UInterchangePipelineBase*> GeneratedPipelines;
	GeneratePipelines(GetPipelinesForSourceData(InSourceData), GeneratedPipelines);
	UInterchangeGenericAssetsPipeline* AssetPipeline = GetGenericAssetPipeline(GeneratedPipelines);
	if (AssetPipeline)
	{
		AssetPipeline->MeshPipeline->bImportStaticMeshes   = false;
		AssetPipeline->MeshPipeline->bImportSkeletalMeshes = false;
		AssetPipeline->MaterialPipeline->MaterialImport    = EInterchangeMaterialImportOption::ImportAsMaterialInstances;
	}

	UInterchangeMegascansPipeline* MegascanPipeline = GetMegascanPipeline(GeneratedPipelines);
	if (MegascanPipeline)
	{
		MegascanPipeline->MegascanImportType = EMegascanImportType::Decal;
	}

	FImportAssetParameters ImportAssetParameters;
	ImportAssetParameters.bIsAutomated      = true;
	ImportAssetParameters.OverridePipelines = TArray<FSoftObjectPath>(GeneratedPipelines);

	UInterchangeManager& InterchangeManager             = UInterchangeManager::GetInterchangeManager();
	const UE::Interchange::FAssetImportResultRef Result = InterchangeManager.ImportAssetAsync(DestinationPath, InSourceData, ImportAssetParameters);
	Result->OnDone(
		[OnDone, Pipelines = MoveTemp(GeneratedPipelines)](const UE::Interchange::FImportResult& Result)
		{
			const UE::Interchange::FImportResult::EStatus Status = Result.GetStatus();
			if (Status == UE::Interchange::FImportResult::EStatus::Done)
			{
				OnDone(Result.GetImportedObjects());
			}
			else if (Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				OnDone({});
			}
			if (Status == UE::Interchange::FImportResult::EStatus::Done || Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				for (UInterchangePipelineBase* const Pipeline : Pipelines)
				{
					Pipeline->RemoveFromRoot();
				}
			}
		}
	);
}

void FQuixelGltfImporter::ImportGltfImperfectionAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone)
{
	const UInterchangeSourceData* InSourceData = UInterchangeManager::CreateSourceData(SourcePath);

	TArray<UInterchangePipelineBase*> GeneratedPipelines;
	GeneratePipelines(GetPipelinesForSourceData(InSourceData), GeneratedPipelines);
	UInterchangeGenericAssetsPipeline* AssetPipeline = GetGenericAssetPipeline(GeneratedPipelines);
	if (AssetPipeline)
	{
		AssetPipeline->MeshPipeline->bImportStaticMeshes                  = false;
		AssetPipeline->MeshPipeline->bImportSkeletalMeshes                = false;
		AssetPipeline->MaterialPipeline->bImportMaterials                 = false;
		AssetPipeline->MaterialPipeline->TexturePipeline->bImportTextures = true;
	}

	UInterchangeMegascansPipeline* MegascanPipeline = GetMegascanPipeline(GeneratedPipelines);
	if (MegascanPipeline)
	{
		MegascanPipeline->MegascanImportType = EMegascanImportType::Imperfection;
	}

	FImportAssetParameters ImportAssetParameters;
	ImportAssetParameters.bIsAutomated      = true;
	ImportAssetParameters.OverridePipelines = TArray<FSoftObjectPath>(GeneratedPipelines);

	UInterchangeManager& InterchangeManager             = UInterchangeManager::GetInterchangeManager();
	const UE::Interchange::FAssetImportResultRef Result = InterchangeManager.ImportAssetAsync(DestinationPath, InSourceData, ImportAssetParameters);
	Result->OnDone(
		[OnDone, Pipelines = MoveTemp(GeneratedPipelines)](const UE::Interchange::FImportResult& Result)
		{
			const UE::Interchange::FImportResult::EStatus Status = Result.GetStatus();
			if (Status == UE::Interchange::FImportResult::EStatus::Done)
			{
				OnDone(Result.GetImportedObjects());
			}
			else if (Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				OnDone({});
			}
			if (Status == UE::Interchange::FImportResult::EStatus::Done || Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				for (UInterchangePipelineBase* const Pipeline : Pipelines)
				{
					Pipeline->RemoveFromRoot();
				}
			}
		}
	);
}

void FQuixelGltfImporter::ImportGltfSurfaceAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone)
{
	const UInterchangeSourceData* InSourceData = UInterchangeManager::CreateSourceData(SourcePath);

	TArray<UInterchangePipelineBase*> GeneratedPipelines;
	GeneratePipelines(GetPipelinesForSourceData(InSourceData), GeneratedPipelines);
	UInterchangeGenericAssetsPipeline* AssetPipeline = GetGenericAssetPipeline(GeneratedPipelines);
	if (AssetPipeline)
	{
		AssetPipeline->MeshPipeline->bImportStaticMeshes   = false;
		AssetPipeline->MeshPipeline->bImportSkeletalMeshes = false;
		AssetPipeline->MaterialPipeline->MaterialImport    = EInterchangeMaterialImportOption::ImportAsMaterialInstances;
	};

	UInterchangeMegascansPipeline* MegascanPipeline = GetMegascanPipeline(GeneratedPipelines);
	if (MegascanPipeline)
	{
		MegascanPipeline->MegascanImportType = EMegascanImportType::Surface;
	}

	FImportAssetParameters ImportAssetParameters;
	ImportAssetParameters.bIsAutomated      = true;
	ImportAssetParameters.OverridePipelines = TArray<FSoftObjectPath>(GeneratedPipelines);

	UInterchangeManager& InterchangeManager             = UInterchangeManager::GetInterchangeManager();
	const UE::Interchange::FAssetImportResultRef Result = InterchangeManager.ImportAssetAsync(DestinationPath, InSourceData, ImportAssetParameters);
	Result->OnDone(
		[OnDone, Pipelines = MoveTemp(GeneratedPipelines)](const UE::Interchange::FImportResult& Result)
		{
			const UE::Interchange::FImportResult::EStatus Status = Result.GetStatus();
			if (Status == UE::Interchange::FImportResult::EStatus::Done)
			{
				OnDone(Result.GetImportedObjects());
			}
			else if (Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				OnDone({});
			}
			if (Status == UE::Interchange::FImportResult::EStatus::Done || Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				for (UInterchangePipelineBase* const Pipeline : Pipelines)
				{
					Pipeline->RemoveFromRoot();
				}
			}
		}
	);
}

void FQuixelGltfImporter::ImportGltfPlantAsset(const FString& SourcePath, const FString& DestinationPath, const bool bBuildNanite, TFunction<void(const TArray<UObject*>&)> OnDone)
{
	const UInterchangeSourceData* InSourceData = UInterchangeManager::CreateSourceData(SourcePath);

	TArray<UInterchangePipelineBase*> GeneratedPipelines;
	GeneratePipelines(GetPipelinesForSourceData(InSourceData), GeneratedPipelines);
	UInterchangeGenericAssetsPipeline* AssetPipeline = GetGenericAssetPipeline(GeneratedPipelines);
	if (AssetPipeline)
	{
		AssetPipeline->MeshPipeline->bImportStaticMeshes                             = true;
		AssetPipeline->MeshPipeline->bImportSkeletalMeshes                           = false;
		//TODO: Move buildNanite into import pipeline
		AssetPipeline->MeshPipeline->bBuildNanite                                    = bBuildNanite;
		AssetPipeline->CommonMeshesProperties->bRecomputeNormals                     = true;
		AssetPipeline->CommonMeshesProperties->bComputeWeightedNormals               = true;
		AssetPipeline->MaterialPipeline->MaterialImport                              = EInterchangeMaterialImportOption::ImportAsMaterialInstances;
		AssetPipeline->MaterialPipeline->TexturePipeline->bFlipNormalMapGreenChannel = true;

		#if (ENGINE_MAJOR_VERSION >=5 && ENGINE_MINOR_VERSION >= 5)
		AssetPipeline->MeshPipeline->bCollision = false;
		#else
		AssetPipeline->MeshPipeline->bImportCollision = false;
		#endif
	}

	UInterchangeMegascansPipeline* MegascanPipeline = GetMegascanPipeline(GeneratedPipelines);
	if (MegascanPipeline)
	{
		MegascanPipeline->MegascanImportType = EMegascanImportType::Plant;
	}

	FImportAssetParameters ImportAssetParameters;
	ImportAssetParameters.bIsAutomated      = true;
	ImportAssetParameters.OverridePipelines = TArray<FSoftObjectPath>(GeneratedPipelines);
	ImportAssetParameters.OnAssetsImportDoneNative.BindLambda(
		[DestinationPath](const TArray<UObject*>& ImportedObjects)
		{
			SetupGlobalFoliageActor(DestinationPath);
		}
	);

	UInterchangeManager& InterchangeManager             = UInterchangeManager::GetInterchangeManager();
	const UE::Interchange::FAssetImportResultRef Result = InterchangeManager.ImportAssetAsync(DestinationPath, InSourceData, ImportAssetParameters);
	Result->OnDone(
		[OnDone, Pipelines = MoveTemp(GeneratedPipelines)](const UE::Interchange::FImportResult& Result)
		{
			const UE::Interchange::FImportResult::EStatus Status = Result.GetStatus();
			if (Status == UE::Interchange::FImportResult::EStatus::Done)
			{
				OnDone(Result.GetImportedObjects());
			}
			else if (Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				OnDone({});
			}
			if (Status == UE::Interchange::FImportResult::EStatus::Done || Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				for (UInterchangePipelineBase* const Pipeline : Pipelines)
				{
					Pipeline->RemoveFromRoot();
				}
			}
		}
	);
}

void FQuixelGltfImporter::ImportGltf3DAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone)
{
	const UInterchangeSourceData* InSourceData = UInterchangeManager::CreateSourceData(SourcePath);

	TArray<UInterchangePipelineBase*> GeneratedPipelines;
	GeneratePipelines(GetPipelinesForSourceData(InSourceData), GeneratedPipelines);
	UInterchangeGenericAssetsPipeline* AssetPipeline = GetGenericAssetPipeline(GeneratedPipelines);
	if (AssetPipeline)
	{
		AssetPipeline->MeshPipeline->bImportStaticMeshes   = true;
		AssetPipeline->MeshPipeline->bImportSkeletalMeshes = false;
		AssetPipeline->MeshPipeline->bBuildNanite          = true;
		AssetPipeline->MaterialPipeline->MaterialImport    = EInterchangeMaterialImportOption::ImportAsMaterialInstances;
	}

	UInterchangeMegascansPipeline* MegascanPipeline = GetMegascanPipeline(GeneratedPipelines);
	if (MegascanPipeline)
	{
		MegascanPipeline->MegascanImportType = EMegascanImportType::Model3D;
	}

	FImportAssetParameters ImportAssetParameters;
	ImportAssetParameters.bIsAutomated      = true;
	ImportAssetParameters.OverridePipelines = TArray<FSoftObjectPath>(GeneratedPipelines);

	UInterchangeManager& InterchangeManager             = UInterchangeManager::GetInterchangeManager();
	const UE::Interchange::FAssetImportResultRef Result = InterchangeManager.ImportAssetAsync(DestinationPath, InSourceData, ImportAssetParameters);
	Result->OnDone(
		[OnDone, Pipelines = MoveTemp(GeneratedPipelines)](const UE::Interchange::FImportResult& Result)
		{
			const UE::Interchange::FImportResult::EStatus Status = Result.GetStatus();
			if (Status == UE::Interchange::FImportResult::EStatus::Done)
			{
				OnDone(Result.GetImportedObjects());
			}
			else if (Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				OnDone({});
			}
			if (Status == UE::Interchange::FImportResult::EStatus::Done || Status == UE::Interchange::FImportResult::EStatus::Invalid)
			{
				for (UInterchangePipelineBase* const Pipeline : Pipelines)
				{
					Pipeline->RemoveFromRoot();
				}
			}
		}
	);
}
