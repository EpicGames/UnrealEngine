// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeManager.h"

class FQuixelGltfImporter
{
private:
	static void SetupGlobalFoliageActor(const FString& ImportPath);

	static TArray<FSoftObjectPath> GetPipelinesForSourceData(const UInterchangeSourceData* InSourceData);
	static void GeneratePipelines(const TArray<FSoftObjectPath>& OriginalPipelines, TArray<UInterchangePipelineBase*>& GeneratedPipelines);
	static class UInterchangeGenericAssetsPipeline* GetGenericAssetPipeline(const TArray<UInterchangePipelineBase*>& GeneratedPipelines);
	static class UInterchangeMegascansPipeline* GetMegascanPipeline(const TArray<UInterchangePipelineBase*>& GeneratedPipelines);

public:
	static void ImportGltfDecalAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone);
	static void ImportGltfImperfectionAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone);
	static void ImportGltfSurfaceAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone);
	static void ImportGltfPlantAsset(const FString& SourcePath, const FString& DestinationPath, const bool bBuildNanite, TFunction<void(const TArray<UObject*>&)> OnDone);
	static void ImportGltf3DAsset(const FString& SourcePath, const FString& DestinationPath, TFunction<void(const TArray<UObject*>&)> OnDone);
};
