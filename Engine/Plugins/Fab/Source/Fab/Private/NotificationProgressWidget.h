// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Styling/SlateStyleMacros.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/INotificationWidget.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Text/STextBlock.h"

class SNotificationProgressWidget : public SCompoundWidget, public INotificationWidget
{
public:
	SLATE_BEGIN_ARGS(SNotificationProgressWidget)
			: _ProgressText(FText::FromString("Downloading Content"))
			, _HasButton(false)
		{}

		SLATE_ARGUMENT(FText, ProgressText);
		SLATE_ARGUMENT(bool, HasButton);
		SLATE_ARGUMENT(FText, ButtonText);
		SLATE_ARGUMENT(FText, ButtonToolTip);
		SLATE_EVENT(FOnClicked, OnButtonClicked);
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		ChildSlot[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(InArgs._ProgressText)
				.AutoWrapText(true)
				.Font(FAppStyle::Get().GetFontStyle("NotificationList.FontBold"))
			]
			+ SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(9)
				[
					SNew(SOverlay)
					+ SOverlay::Slot()
					.VAlign(VAlign_Center)
					[
						SAssignNew(ProgressBar, SProgressBar)
						.Percent(100.0f)
					]
					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SAssignNew(PercentText, STextBlock)
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4.0f, 0.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				[
					SAssignNew(CancelButton, SButton)
					.Text(InArgs._ButtonText)
					.IsEnabled(InArgs._HasButton)
					.ToolTipText(InArgs._ButtonToolTip)
					.OnClicked(InArgs._OnButtonClicked)
				]
			]
		];
	}

	void SetProgressPercent(const float Percent)
	{
		ProgressBar->SetPercent(Percent / 100);
		PercentText->SetText(FText::AsPercent(Percent / 100.0f));
		PercentText->SetColorAndOpacity(Percent <= 55.0f ? FLinearColor::White : FLinearColor::Black);
	}

	void DisableCancel()
	{
		CancelButton->SetEnabled(false);
	}

	/** INotificationWidget interface */
	virtual void OnSetCompletionState(SNotificationItem::ECompletionState) override
	{}

	virtual TSharedRef<SWidget> AsWidget() override
	{
		return AsShared();
	}

private:
	TSharedPtr<SProgressBar> ProgressBar;
	TSharedPtr<STextBlock> PercentText;
	TSharedPtr<SButton> CancelButton;
};
