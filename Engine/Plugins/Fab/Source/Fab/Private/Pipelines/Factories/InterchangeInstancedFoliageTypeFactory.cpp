// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeInstancedFoliageTypeFactory.h"

#include "Engine/StaticMesh.h"
#include "FoliageType_InstancedStaticMesh.h"
#include "InterchangeSourceData.h"
#include "InterchangeStaticMeshFactoryNode.h"

#include "Nodes/InterchangeBaseNodeContainer.h"

#include "Pipelines/Nodes/InterchangeInstancedFoliageTypeFactoryNode.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeInstancedFoliageTypeFactory)

UClass* UInterchangeInstancedFoliageTypeFactory::GetFactoryClass() const
{
	return UFoliageType_InstancedStaticMesh::StaticClass();
}

UInterchangeFactoryBase::FImportAssetResult UInterchangeInstancedFoliageTypeFactory::BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeInstancedFoliageTypeFactory::BeginImportAsset_GameThread);

	Super::BeginImportAsset_GameThread(Arguments);
	FImportAssetResult ImportAssetResult;

	UFoliageType_InstancedStaticMesh* ImportedObject = nullptr;

	auto CouldNotCreateDemoObjectLog = [this, &Arguments, &ImportAssetResult](const FString& Info)
	{
		UInterchangeResultError_Generic* Message = AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName                 = Arguments.SourceData->GetFilename();
		Message->DestinationAssetName            = Arguments.AssetName;
		Message->AssetType                       = GetFactoryClass();
		Message->Text                            = FText::FromString(
			FString::Printf(TEXT("UInterchangeInstancedFoliageTypeFactory: Could not create UFoliageType_InstancedStaticMesh asset %s. Reason: %s"), *Arguments.AssetName, *Info)
		);
		ImportAssetResult.bIsFactorySkipAsset = true;
	};

	const UInterchangeInstancedFoliageTypeFactoryNode* DemoObjectFactoryNode = Cast<UInterchangeInstancedFoliageTypeFactoryNode>(Arguments.AssetNode);
	if (!DemoObjectFactoryNode)
	{
		CouldNotCreateDemoObjectLog(TEXT("Asset node parameter is null."));
		return ImportAssetResult;
	}

	const UClass* InstancedFoliageTypeClass = Arguments.AssetNode->GetObjectClass();
	if (!InstancedFoliageTypeClass || !InstancedFoliageTypeClass->IsChildOf(UFoliageType_InstancedStaticMesh::StaticClass()))
	{
		CouldNotCreateDemoObjectLog(TEXT("Asset node parameter class doesn't derive from UFoliageType_InstancedStaticMesh."));
		return ImportAssetResult;
	}

	FSoftObjectPath ReferenceObject;
	if (Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
	{
		ImportedObject = Cast<UFoliageType_InstancedStaticMesh>(ReferenceObject.TryLoad());
	}
	if (!ImportedObject)
	{
		ImportedObject = NewObject<UFoliageType_InstancedStaticMesh>(Arguments.Parent, InstancedFoliageTypeClass, *Arguments.AssetName, RF_Public | RF_Standalone);
	}

	if (!ImportedObject)
	{
		CouldNotCreateDemoObjectLog(TEXT("UFoliageType_InstancedStaticMeshObject creation fail."));
		return ImportAssetResult;
	}

	ImportAssetResult.ImportedObject = ImportedObject;

	return ImportAssetResult;
}

void UInterchangeInstancedFoliageTypeFactory::SetupObject_GameThread(const FSetupObjectParams& Arguments)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeInstancedFoliageTypeFactory::SetupObject_GameThread);

	check(IsInGameThread());

	UFoliageType_InstancedStaticMesh* InstancedFoliageType = Cast<UFoliageType_InstancedStaticMesh>(Arguments.ImportedObject);
	if (!InstancedFoliageType)
	{
		return;
	}

	#if WITH_EDITOR
	{
		InstancedFoliageType->PreEditChange(nullptr);

		const UInterchangeFactoryBaseNode* FactoryNode = Arguments.FactoryNode;
		FactoryNode->ApplyAllCustomAttributeToObject(InstancedFoliageType);
		if (const UInterchangeInstancedFoliageTypeFactoryNode* InstancedFoliageTypeFactoryNode = Cast<UInterchangeInstancedFoliageTypeFactoryNode>(FactoryNode))
		{
			if (FString StaticMeshNodeUid; InstancedFoliageTypeFactoryNode->GetCustomStaticMesh(StaticMeshNodeUid))
			{
				if (const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = Cast<UInterchangeStaticMeshFactoryNode>(Arguments.NodeContainer->GetNode(StaticMeshNodeUid)))
				{
					if (FSoftObjectPath StaticMeshPath; StaticMeshFactoryNode->GetCustomReferenceObject(StaticMeshPath))
					{
						InstancedFoliageType->SetStaticMesh(Cast<UStaticMesh>(StaticMeshPath.TryLoad()));
					}
				}
			}

			FVector2f Scale;
			if (InstancedFoliageTypeFactoryNode->GetCustomScaleX(Scale))
			{
				InstancedFoliageType->ScaleX = FFloatInterval(Scale.X, Scale.Y);
			}
			if (InstancedFoliageTypeFactoryNode->GetCustomScaleY(Scale))
			{
				InstancedFoliageType->ScaleY = FFloatInterval(Scale.X, Scale.Y);
			}
			if (InstancedFoliageTypeFactoryNode->GetCustomScaleZ(Scale))
			{
				InstancedFoliageType->ScaleZ = FFloatInterval(Scale.X, Scale.Y);
			}
		}
	}
	#endif

	Super::SetupObject_GameThread(Arguments);
}
