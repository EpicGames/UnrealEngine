// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangeFactoryBase.h"

#include "InterchangeInstancedFoliageTypeFactory.generated.h"

UCLASS()
class UInterchangeInstancedFoliageTypeFactory : public UInterchangeFactoryBase
{
	GENERATED_BODY()

public:
	virtual UClass* GetFactoryClass() const override;
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() override { return EInterchangeFactoryAssetType::Meshes; }
	virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments) override;
	virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments) override;
};
