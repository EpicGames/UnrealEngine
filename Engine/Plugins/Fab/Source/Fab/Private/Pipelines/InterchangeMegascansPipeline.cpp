// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeMegascansPipeline.h"

#include "FoliageType.h"
#include "InterchangeCommonPipelineDataFactoryNode.h"
#include "InterchangeGenericAssetsPipelineSharedSettings.h"
#include "InterchangeMaterialFactoryNode.h"
#include "InterchangeMaterialInstanceNode.h"
#include "InterchangeMeshFactoryNode.h"
#include "InterchangeMeshNode.h"
#include "InterchangePipelineHelper.h"
#include "InterchangePipelineMeshesUtilities.h"
#include "InterchangeSceneNode.h"
#include "InterchangeShaderGraphNode.h"
#include "InterchangeStaticMeshFactoryNode.h"
#include "InterchangeStaticMeshLodDataNode.h"

#include "Engine/StaticMesh.h"
#include "InterchangeTextureFactoryNode.h"

#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Mesh/InterchangeMeshHelper.h"

#include "Nodes/InterchangeInstancedFoliageTypeFactoryNode.h"
#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "Nodes/InterchangeSourceNode.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeMegascansPipeline)

#define MEGASCAN_BASE_KEY									TEXT("Megascan")

#define MEGASCAN_MATERIAL_KEY							    MEGASCAN_BASE_KEY TEXT(".Material")

#define MEGASCAN_MATERIAL_TYPE_KEY							MEGASCAN_MATERIAL_KEY TEXT(".Type")

#define MEGASCAN_MATERIAL_BLEND_MODE_KEY				    MEGASCAN_MATERIAL_KEY TEXT(".BlendMode")
#define MEGASCAN_MATERIAL_BLEND_MODE_VALUE_KEY				MEGASCAN_MATERIAL_BLEND_MODE_KEY TEXT(".Value")
#define MEGASCAN_MATERIAL_BLEND_MODE_OVERRIDE_KEY		    MEGASCAN_MATERIAL_BLEND_MODE_KEY TEXT(".Override")

#define MEGASCAN_MATERIAL_DISPLACEMENT_KEY					MEGASCAN_MATERIAL_KEY TEXT(".Displacement")
#define MEGASCAN_MATERIAL_DISPLACEMENT_OVERRIDE_KEY			MEGASCAN_MATERIAL_DISPLACEMENT_KEY TEXT(".Override")
#define MEGASCAN_MATERIAL_DISPLACEMENT_MAGNITUDE_KEY		MEGASCAN_MATERIAL_DISPLACEMENT_KEY TEXT(".Magnitude")
#define MEGASCAN_MATERIAL_DISPLACEMENT_CENTER_KEY			MEGASCAN_MATERIAL_DISPLACEMENT_KEY TEXT(".Center")

#define MEGASCAN_MESH_KEY									MEGASCAN_BASE_KEY TEXT(".Mesh")
#define MEGASCAN_MESH_GENERATE_DISTANCE_FIELD_KEY			MEGASCAN_MESH_KEY TEXT(".GenerateDistanceField")
#define MEGASCAN_MESH_AUTO_COMPUTE_LOD_SCREEN_SIZE_KEY		MEGASCAN_MESH_KEY TEXT(".AutoComputeLODScreenSize")
#define MEGASCAN_MESH_NANITE_SETTINGS_KEY				    MEGASCAN_MESH_KEY TEXT(".Nanite")
#define MEGASCAN_MESH_NANITE_PRESERVE_AREA_KEY				MEGASCAN_MESH_NANITE_SETTINGS_KEY TEXT(".ShapePreservation")

UInterchangeMegascansPipeline::UInterchangeMegascansPipeline()
	: MegascanImportType(EMegascanImportType::Model3D)
	, MegascansMaterialParentSettings(GetMutableDefault<UMegascansMaterialParentSettings>())
{}

void UInterchangeMegascansPipeline::ExecutePipeline(
	UInterchangeBaseNodeContainer* NodeContainer,
	const TArray<UInterchangeSourceData*>& SourceDatas
	#if ENGINE_MAJOR_VERSION== 5 && ENGINE_MINOR_VERSION > 3
	,
	const FString& ContentBasePath
	#endif
)
{
	Super::ExecutePipeline(
		NodeContainer,
		SourceDatas
		#if ENGINE_MAJOR_VERSION== 5 && ENGINE_MINOR_VERSION > 3
		,
		ContentBasePath
		#endif
	);

	BaseNodeContainer = NodeContainer;

	const UInterchangeSourceData* const* GltfSourceData = SourceDatas.FindByPredicate(
		[](const UInterchangeSourceData* SourceData)
		{
			return FPaths::GetExtension(SourceData->GetFilename()) == "gltf";
		}
	);
	if (!GltfSourceData)
	{
		return;
	}

	if (!LoadGltfSource((*GltfSourceData)->GetFilename()))
	{
		return;
	}

	if (const TSharedPtr<FJsonObject>* GltfExtras; GltfJson->TryGetObjectField(TEXT("extras"), GltfExtras))
	{
		GltfExtras->Get()->TryGetNumberField(TEXT("tier"), reinterpret_cast<int8&>(MegascanAssetTier));
	}

	TextureFactoryNodes          = GetNodesOfType<UInterchangeTextureFactoryNode>();
	StaticMeshFactoryNodes       = GetNodesOfType<UInterchangeStaticMeshFactoryNode>();
	MaterialInstanceFactoryNodes = GetNodesOfType<UInterchangeMaterialInstanceFactoryNode>();

	ForEachGltfTexture(
		[&](const FString& TextureName, const TSharedPtr<FJsonObject>& Texture)
		{
			UInterchangeTextureFactoryNode* TextureFactoryNode = FindTextureFactoryNodeByName(TextureName);
			if (TextureFactoryNode == nullptr)
			{
				return;
			}

			if (const TSharedPtr<FJsonObject>* TextureExtras; Texture->TryGetObjectField(TEXT("extras"), TextureExtras))
			{
				SetupTextureParams(TextureFactoryNode, *TextureExtras);
			}
		}
	);

	ForEachGltfMaterial(
		[&](const FString& MaterialName, const TSharedPtr<FJsonObject>& Material)
		{
			FString MaterialType;
			if (MegascanImportType == EMegascanImportType::Plant && MaterialName.EndsWith("_Billboard", ESearchCase::CaseSensitive))
				MaterialType = "billboard";

			UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = FindMaterialInstanceFactoryNodeByName(MaterialName);
			if (MaterialInstanceFactoryNode == nullptr)
			{
				return;
			}

			SetupMaterial(MaterialInstanceFactoryNode);
			if (const TSharedPtr<FJsonObject>* MaterialExtras; Material->TryGetObjectField(TEXT("extras"), MaterialExtras))
			{
				MaterialExtras->Get()->TryGetStringField(TEXT("type"), MaterialType);
				SetupMaterialParams(MaterialInstanceFactoryNode, *MaterialExtras);
			}
			SetupMaterialParents(MaterialInstanceFactoryNode, MaterialType);
		}
	);

	ForEachGltfMesh(
		[&](const FString& MeshName, const TSharedPtr<FJsonObject>& Mesh)
		{
			UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode = FindStaticMeshFactoryNodeByName(MeshName);
			if (StaticMeshFactoryNode == nullptr)
			{
				return;
			}

			SetupStaticMesh(StaticMeshFactoryNode);
			if (const TSharedPtr<FJsonObject>* MeshExtras; Mesh->TryGetObjectField(TEXT("extras"), MeshExtras))
			{
				SetupStaticMeshParams(StaticMeshFactoryNode, *MeshExtras);
			}
		}
	);

	StaticMeshFactoryNodes = GetNodesOfType<UInterchangeStaticMeshFactoryNode>();
	for (UInterchangeStaticMeshFactoryNode* MeshFactoryNode : StaticMeshFactoryNodes)
	{
		UE::Interchange::MeshesUtilities::ReorderSlotMaterialDependencies(*MeshFactoryNode, *BaseNodeContainer);
	}
}

void UInterchangeMegascansPipeline::ExecutePostFactoryPipeline(
	const UInterchangeBaseNodeContainer* NodeContainer,
	const FString& NodeKey,
	UObject* CreatedAsset,
	const bool bIsAReimport
)
{
	Super::ExecutePostFactoryPipeline(NodeContainer, NodeKey, CreatedAsset, bIsAReimport);

	if (MegascanImportType == EMegascanImportType::Plant && MegascanAssetTier > EMegascanImportTier::Raw)
	{
		if (UStaticMesh* ImportedMesh = Cast<UStaticMesh>(CreatedAsset))
		{
			const float BillboardScreenSizes[] = {
				0.03f,
				0.05f,
				0.10f
			};
			int32 Index = 0;
			for (const int32 MaxIndex = ImportedMesh->GetNumSourceModels(); Index < MaxIndex; ++Index)
			{
				ImportedMesh->GetSourceModel(Index).ScreenSize = FMath::Pow(0.75f, Index);
			}
			if (const int ScreenSizeIndex = static_cast<int8>(MegascanAssetTier) - 1; ScreenSizeIndex >= 0)
				ImportedMesh->GetSourceModel(Index - 1).ScreenSize = BillboardScreenSizes[ScreenSizeIndex];
		}
	}

	if (bVirtualTexturesImported)
	{
		return;
	}

	if (const UTexture* ImportedTexture = Cast<UTexture>(CreatedAsset))
	{
		bVirtualTexturesImported |= ImportedTexture->VirtualTextureStreaming;
	}

	if (bVirtualTexturesImported)
	{
		for (UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode : MaterialInstanceFactoryNodes)
		{
			UpdateParentMaterial(MaterialInstanceFactoryNode, true, true);
		}
	}
}

bool UInterchangeMegascansPipeline::LoadGltfSource(const FString& SourceFile)
{
	if (FString GltfFileData; FFileHelper::LoadFileToString(GltfFileData, *SourceFile))
	{
		GltfJson = MakeShareable(new FJsonObject);
		return FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(GltfFileData), GltfJson);
	}
	return false;
}

void UInterchangeMegascansPipeline::ForEachGltfMaterial(const TFunction<void(const FString&, const TSharedPtr<FJsonObject>&)>& Callback) const
{
	if (GltfJson == nullptr) { return; }
	const TArray<TSharedPtr<FJsonValue>>& Materials = GltfJson->GetArrayField(TEXT("materials"));
	for (const TSharedPtr<FJsonValue>& Material : Materials)
	{
		const TSharedPtr<FJsonObject>& MaterialObject = Material->AsObject();
		const FString MaterialName                    = MaterialObject->GetStringField(TEXT("name"));
		Callback(MaterialName, MaterialObject);
	}
}

void UInterchangeMegascansPipeline::ForEachGltfTexture(const TFunction<void(const FString&, const TSharedPtr<FJsonObject>&)>& Callback)
{
	if (GltfJson == nullptr) { return; }
	const TArray<TSharedPtr<FJsonValue>>& Images = GltfJson->GetArrayField(TEXT("images"));
	for (const TSharedPtr<FJsonValue>& Image : Images)
	{
		const TSharedPtr<FJsonObject>& ImageObject = Image->AsObject();
		const FString ImageName                    = ImageObject->GetStringField(TEXT("name"));
		Callback(ImageName, ImageObject);
	}
}

void UInterchangeMegascansPipeline::ForEachGltfMesh(const TFunction<void(const FString&, const TSharedPtr<FJsonObject>&)>& Callback)
{
	if (GltfJson == nullptr) { return; }
	const TArray<TSharedPtr<FJsonValue>>& MeshNodes = GltfJson->GetArrayField(TEXT("nodes"));
	for (const TSharedPtr<FJsonValue>& MeshNode : MeshNodes)
	{
		const TSharedPtr<FJsonObject>& MeshNodeObject = MeshNode->AsObject();
		const FString MeshNodeName                    = MeshNodeObject->GetStringField(TEXT("name"));
		Callback(MeshNodeName, MeshNodeObject);
	}
}

UInterchangeTextureFactoryNode* UInterchangeMegascansPipeline::FindTextureFactoryNodeByName(const FString& DisplayName) const
{
	UInterchangeTextureFactoryNode* const* const FoundNode = TextureFactoryNodes.FindByPredicate(
		[&DisplayName](const UInterchangeTextureFactoryNode* Node)
		{
			return Node->GetDisplayLabel() == DisplayName;
		}
	);
	return FoundNode ? *FoundNode : nullptr;
}

UInterchangeStaticMeshFactoryNode* UInterchangeMegascansPipeline::FindStaticMeshFactoryNodeByName(const FString& DisplayName) const
{
	UInterchangeStaticMeshFactoryNode* const* const FoundNode = StaticMeshFactoryNodes.FindByPredicate(
		[&DisplayName](const UInterchangeStaticMeshFactoryNode* Node)
		{
			return Node->GetDisplayLabel() == DisplayName;
		}
	);
	return FoundNode ? *FoundNode : nullptr;
}

UInterchangeMaterialInstanceFactoryNode* UInterchangeMegascansPipeline::FindMaterialInstanceFactoryNodeByName(const FString& DisplayName) const
{
	UInterchangeMaterialInstanceFactoryNode* const* const FoundNode = MaterialInstanceFactoryNodes.FindByPredicate(
		[&DisplayName](const UInterchangeMaterialInstanceFactoryNode* Node)
		{
			return Node->GetDisplayLabel() == DisplayName;
		}
	);
	return FoundNode ? *FoundNode : nullptr;
}

const TSharedPtr<FJsonObject>* UInterchangeMegascansPipeline::GetMaterialAtIndex(uint32 Index) const
{
	TSharedPtr<FJsonObject>* Material              = nullptr;
	const TArray<TSharedPtr<FJsonValue>> Materials = GltfJson->GetArrayField(TEXT("materials"));
	if (Materials.IsValidIndex(Index))
	{
		Materials[Index]->TryGetObject(Material);
	}
	return Material;
}

EMegascanMaterialType UInterchangeMegascansPipeline::GetMegascanMaterialType(const UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const
{
	int32 MaterialType = 0;
	MaterialInstanceFactoryNode->GetInt32Attribute(MEGASCAN_MATERIAL_TYPE_KEY, MaterialType);

	return static_cast<EMegascanMaterialType>(MaterialType);
}

bool UInterchangeMegascansPipeline::SetMegascanMaterialType(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, EMegascanMaterialType MaterialType) const
{
	return MaterialInstanceFactoryNode->AddInt32Attribute(MEGASCAN_MATERIAL_TYPE_KEY, static_cast<int32>(MaterialType));
}

bool UInterchangeMegascansPipeline::UpdateParentMaterial(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, bool bVTMaterial, bool bUpdateReferencedObject)
{
	const EMegascanMaterialType MaterialType = GetMegascanMaterialType(MaterialInstanceFactoryNode);
	if (MegascansMaterialParentSettings)
	{
		if (const FMegascanMaterialPair* const ParentMaterialPair = MegascansMaterialParentSettings->MaterialParents.Find(MaterialType))
		{
			const TSoftObjectPtr<UMaterialInterface> ParentMaterial = bVTMaterial ? ParentMaterialPair->VTMaterial : ParentMaterialPair->StandardMaterial;

			if (bUpdateReferencedObject)
			{
				if (FSoftObjectPath MaterialInstancePath; MaterialInstanceFactoryNode->GetCustomReferenceObject(MaterialInstancePath))
				{
					UObject* ImportedObject = MaterialInstancePath.TryLoad();
					if (UMaterialInstanceConstant* Material = Cast<UMaterialInstanceConstant>(ImportedObject))
					{
						Material->SetParentEditorOnly(ParentMaterial.LoadSynchronous());
					}
				}
			}

			return MaterialInstanceFactoryNode->SetCustomParent(ParentMaterial.ToString());
		}
	}
	return false;
}

FString UInterchangeMegascansPipeline::GetStaticMeshLodDataNodeUid(const UInterchangeMeshFactoryNode* MeshFactoryNode, int32 LodIndex)
{
	const FString MeshFactoryUid = MeshFactoryNode->GetUniqueID();
	const FString LODDataPrefix  = TEXT("\\LodData") + (LodIndex > 0 ? FString::FromInt(LodIndex) : TEXT(""));
	return LODDataPrefix + MeshFactoryUid;
}

FString UInterchangeMegascansPipeline::GetStaticMeshLodDataNodeDisplayName(int32 LodIndex)
{
	return TEXT("LodData") + FString::FromInt(LodIndex);
}

void UInterchangeMegascansPipeline::SetupMeshLod(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const UInterchangeSceneNode* SceneNode, int32 LodIndex)
{
	const FString StaticMeshFactoryUid     = StaticMeshFactoryNode->GetUniqueID();
	const FString StaticMeshLodDataNodeUid = GetStaticMeshLodDataNodeUid(StaticMeshFactoryNode, LodIndex);

	UInterchangeStaticMeshLodDataNode* StaticMeshLodDataNode = Cast<UInterchangeStaticMeshLodDataNode>(BaseNodeContainer->GetFactoryNode(StaticMeshLodDataNodeUid));
	if (StaticMeshLodDataNode == nullptr)
	{
		StaticMeshLodDataNode = NewObject<UInterchangeStaticMeshLodDataNode>(BaseNodeContainer, NAME_None);
		BaseNodeContainer->SetupNode(StaticMeshLodDataNode, StaticMeshLodDataNodeUid, GetStaticMeshLodDataNodeDisplayName(LodIndex), EInterchangeNodeContainerType::FactoryData, StaticMeshFactoryUid);
		StaticMeshFactoryNode->AddLodDataUniqueId(StaticMeshLodDataNodeUid);
	}

	const FString SceneNodeUid = SceneNode->GetUniqueID();

	// UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(SceneNode, StaticMeshFactoryNode, true);
	SceneNode->AddTargetNodeUid(StaticMeshFactoryUid);

	FString MeshNodeUid;
	SceneNode->GetCustomAssetInstanceUid(MeshNodeUid);
	if (const UInterchangeMeshNode* MeshNode = Cast<UInterchangeMeshNode>(BaseNodeContainer->GetNode(MeshNodeUid)))
	{
		bool bBakeMeshes = false;
		bool bBakePivotMeshes = false;

		if (CommonMeshesProperties.IsValid())
		{
			bBakeMeshes = CommonMeshesProperties->bBakeMeshes;
			bBakePivotMeshes = CommonMeshesProperties->bBakePivotMeshes;
		}

		FTransform GlobalOffsetTransform = FTransform::Identity;
        if (UInterchangeCommonPipelineDataFactoryNode* CommonPipelineDataFactoryNode = UInterchangeCommonPipelineDataFactoryNode::GetUniqueInstance(BaseNodeContainer))
        {
            CommonPipelineDataFactoryNode->GetCustomGlobalOffsetTransform(GlobalOffsetTransform);
        }

		FTransform CompleteSceneNodeTransform = UE::Interchange::Private::MeshHelper::CalculateCompleteSceneNodeTransform(BaseNodeContainer, SceneNode, bBakeMeshes, bBakePivotMeshes, GlobalOffsetTransform);
		StaticMeshLodDataNode->AddLODMeshData(FInterchangeLODMeshData(MeshNodeUid, CompleteSceneNodeTransform));
		StaticMeshFactoryNode->AddTargetNodeUid(MeshNodeUid);

		TMap<FString, FString> SlotMaterialDependencies;
		MeshNode->GetSlotMaterialDependencies(SlotMaterialDependencies);
		UE::Interchange::MeshesUtilities::ApplySlotMaterialDependencies(
			*StaticMeshFactoryNode,
			SlotMaterialDependencies,
			*BaseNodeContainer
			#if ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION > 3
			,
			nullptr
			#endif
		);
	}
}

void UInterchangeMegascansPipeline::SetFoliageType(const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode)
{
	const FString StaticMeshFactoryNodeUid    = StaticMeshFactoryNode->GetUniqueID();
	const FString FoliageTypeNodeUid          = UInterchangeInstancedFoliageTypeFactoryNode::GetNodeUidFromStaticMeshFactoryUid(StaticMeshFactoryNodeUid);
	const FString FoliageTypeNodeDisplayLabel = StaticMeshFactoryNode->GetDisplayLabel().Replace(TEXT("SM_"), TEXT("FT_"), ESearchCase::CaseSensitive);

	UInterchangeInstancedFoliageTypeFactoryNode* InstancedFoliageTypeFactoryNode = NewObject<UInterchangeInstancedFoliageTypeFactoryNode>(BaseNodeContainer, NAME_None);
	BaseNodeContainer->SetupNode(InstancedFoliageTypeFactoryNode, FoliageTypeNodeUid, FoliageTypeNodeDisplayLabel, EInterchangeNodeContainerType::FactoryData, StaticMeshFactoryNodeUid);

	UInterchangeSourceNode* SourceNode = UInterchangeSourceNode::FindOrCreateUniqueInstance(BaseNodeContainer);
	UE::Interchange::PipelineHelper::FillSubPathFromSourceNode(InstancedFoliageTypeFactoryNode, SourceNode);

	StaticMeshFactoryNode->AddTargetNodeUid(FoliageTypeNodeUid);
	InstancedFoliageTypeFactoryNode->AddTargetNodeUid(StaticMeshFactoryNodeUid);

	InstancedFoliageTypeFactoryNode->AddFactoryDependencyUid(StaticMeshFactoryNodeUid);

	InstancedFoliageTypeFactoryNode->SetCustomSubPath("FoliageTypes");
	InstancedFoliageTypeFactoryNode->SetCustomStaticMesh(StaticMeshFactoryNodeUid);
	InstancedFoliageTypeFactoryNode->SetCustomScaling(EFoliageScaling::Free);
	InstancedFoliageTypeFactoryNode->SetCustomScaleX(FVector2f(0.8f, 1.2f));
	InstancedFoliageTypeFactoryNode->SetCustomScaleY(FVector2f(0.8f, 1.2f));
	InstancedFoliageTypeFactoryNode->SetCustomScaleZ(FVector2f(0.8f, 1.2f));
	InstancedFoliageTypeFactoryNode->SetCustomAlignToNormal(false);
	InstancedFoliageTypeFactoryNode->SetCustomRandomYaw(true);
	InstancedFoliageTypeFactoryNode->SetCustomRandomPitchAngle(3.0f);
	InstancedFoliageTypeFactoryNode->SetCustomAffectDistanceFieldLighting(false);
	if (MegascanAssetTier != EMegascanImportTier::Invalid)
		InstancedFoliageTypeFactoryNode->SetCustomWorldPositionOffsetDisableDistance(5000 - (1000 * static_cast<int8>(MegascanAssetTier)));
}

void UInterchangeMegascansPipeline::SetupTextureParams(UInterchangeTextureFactoryNode* TextureFactoryNode, const TSharedPtr<FJsonObject>& TextureParams)
{
	if (FString CompressionSettings; TextureParams->TryGetStringField(TEXT("compression"), CompressionSettings))
	{
		if (CompressionSettings == "mask")
			TextureFactoryNode->SetCustomCompressionSettings(TC_Masks);
		else if (CompressionSettings == "displacement" || CompressionSettings == "alpha")
			TextureFactoryNode->SetCustomCompressionSettings(TC_Alpha);
	}

	if (FString MigGenSettings; TextureParams->TryGetStringField(TEXT("mipgen"), MigGenSettings))
	{
		if (MigGenSettings == "sharpen_4")
			TextureFactoryNode->SetCustomMipGenSettings(TMGS_Sharpen4);
		else if (MigGenSettings == "sharpen_6")
			TextureFactoryNode->SetCustomMipGenSettings(TMGS_Sharpen6);
	}

	if (const TArray<TSharedPtr<FJsonValue>>* AlphaCoverage; TextureParams->TryGetArrayField(TEXT("alphaCoverage"), AlphaCoverage))
	{
		if (AlphaCoverage->Num() >= 4)
		{
			TextureFactoryNode->SetCustomAlphaCoverageThresholds(
				FVector4(
					AlphaCoverage->GetData()[0]->AsNumber(),
					AlphaCoverage->GetData()[1]->AsNumber(),
					AlphaCoverage->GetData()[2]->AsNumber(),
					AlphaCoverage->GetData()[3]->AsNumber()
				)
			);
		}
	}

	if (bool bScaleMips; TextureParams->TryGetBoolField(TEXT("scaleMips"), bScaleMips))
	{
		TextureFactoryNode->SetCustombDoScaleMipsForAlphaCoverage(bScaleMips);
	}

	if (FString TextureSlot; TextureParams->TryGetStringField(TEXT("textureSlot"), TextureSlot))
	{
		int32 MaterialIndex = 0;
		TextureParams->TryGetNumberField(TEXT("materialIndex"), MaterialIndex);
		if (const TSharedPtr<FJsonObject>* MaterialObject = GetMaterialAtIndex(MaterialIndex))
		{
			const FString MaterialName = MaterialObject->Get()->GetStringField(TEXT("name"));
			if (UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode = FindMaterialInstanceFactoryNodeByName(MaterialName))
			{
				const FString ParameterName     = UInterchangeShaderPortsAPI::MakeInputValueKey(TextureSlot);
				const FString TextureFactoryUid = TextureFactoryNode->GetUniqueID();
				MaterialInstanceFactoryNode->AddStringAttribute(ParameterName, TextureFactoryUid);
				MaterialInstanceFactoryNode->AddFactoryDependencyUid(TextureFactoryUid);
			}
		}
	}
}

void UInterchangeMegascansPipeline::SetupStaticMesh(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode)
{
	if (MegascanImportType == EMegascanImportType::Plant)
	{
		StaticMeshFactoryNode->SetCustomMinLightmapResolution(128);
		StaticMeshFactoryNode->SetAttribute(MEGASCAN_MESH_GENERATE_DISTANCE_FIELD_KEY, true);
		StaticMeshFactoryNode->AddApplyAndFillDelegates<bool>(
			MEGASCAN_MESH_GENERATE_DISTANCE_FIELD_KEY,
			StaticMeshFactoryNode->GetObjectClass(),
			"bGenerateMeshDistanceField"
		);

		if (MegascanAssetTier == EMegascanImportTier::Raw)
		{
			StaticMeshFactoryNode->SetAttribute(MEGASCAN_MESH_NANITE_PRESERVE_AREA_KEY, true);
			StaticMeshFactoryNode->AddApplyAndFillDelegates<bool>(
				MEGASCAN_MESH_NANITE_PRESERVE_AREA_KEY,
				StaticMeshFactoryNode->GetObjectClass(),
				"NaniteSettings.ShapePreservation"
			);
		}
		else
		{
			StaticMeshFactoryNode->SetAttribute(MEGASCAN_MESH_AUTO_COMPUTE_LOD_SCREEN_SIZE_KEY, false);
			StaticMeshFactoryNode->AddApplyAndFillDelegates<bool>(
				MEGASCAN_MESH_AUTO_COMPUTE_LOD_SCREEN_SIZE_KEY,
				StaticMeshFactoryNode->GetObjectClass(),
				"bAutoComputeLODScreenSize"
			);
		}

		const FString StaticMeshDisplayName = StaticMeshFactoryNode->GetDisplayLabel();
		if (!StaticMeshDisplayName.Contains(TEXT("_LOD"), ESearchCase::CaseSensitive, ESearchDir::FromEnd))
		{
			SetFoliageType(StaticMeshFactoryNode);
		}
	}
}

void UInterchangeMegascansPipeline::SetupStaticMeshParams(const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const TSharedPtr<FJsonObject>& MeshParams)
{
	if (const TSharedPtr<FJsonObject>* LodInfo; MeshParams->TryGetObjectField(TEXT("lod"), LodInfo))
	{
		const FString StaticMeshFactoryUid = StaticMeshFactoryNode->GetUniqueID();
		const int32 LodIndex               = LodInfo->Get()->GetNumberField(TEXT("index"));
		const FString LodMeshName          = LodInfo->Get()->GetStringField(TEXT("mesh"));

		UInterchangeStaticMeshFactoryNode* ParentStaticMeshFactoryNode = FindStaticMeshFactoryNodeByName(LodMeshName);
		if (ParentStaticMeshFactoryNode == nullptr)
		{
			return;
		}

		const UInterchangeSceneNode* SceneNode = FindNodeOfTypeByName<UInterchangeSceneNode>(StaticMeshFactoryNode->GetDisplayLabel());
		SetupMeshLod(ParentStaticMeshFactoryNode, SceneNode, LodIndex);

		BaseNodeContainer->RemoveFactoryNode(StaticMeshFactoryUid);
	}
}

void UInterchangeMegascansPipeline::SetupMaterial(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode)
{
	if (MegascanImportType == EMegascanImportType::Plant && MegascanAssetTier == EMegascanImportTier::Raw)
	{
		MaterialInstanceFactoryNode->SetAttribute(MEGASCAN_MATERIAL_BLEND_MODE_OVERRIDE_KEY, true);
		MaterialInstanceFactoryNode->AddApplyAndFillDelegates<bool>(
			MEGASCAN_MATERIAL_BLEND_MODE_OVERRIDE_KEY,
			UMaterialInstanceConstant::StaticClass(),
			"BasePropertyOverrides.bOverride_BlendMode"
		);
		MaterialInstanceFactoryNode->SetAttribute(MEGASCAN_MATERIAL_BLEND_MODE_VALUE_KEY, static_cast<int>(BLEND_Opaque));
		MaterialInstanceFactoryNode->AddApplyAndFillDelegates<int>(
			MEGASCAN_MATERIAL_BLEND_MODE_VALUE_KEY,
			MaterialInstanceFactoryNode->GetObjectClass(),
			"BasePropertyOverrides.BlendMode"
		);
	}
}

void UInterchangeMegascansPipeline::SetupMaterialParams(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, const TSharedPtr<FJsonObject>& MaterialParams)
{
	if (const TSharedPtr<FJsonObject>* Overrides; MaterialParams->TryGetObjectField(TEXT("overrides"), Overrides))
	{
		if (const TSharedPtr<FJsonObject>* Displacement; Overrides->Get()->TryGetObjectField(TEXT("displacement"), Displacement))
		{
			MaterialInstanceFactoryNode->SetAttribute(MEGASCAN_MATERIAL_DISPLACEMENT_OVERRIDE_KEY, true);
			MaterialInstanceFactoryNode->AddApplyAndFillDelegates<bool>(
				MEGASCAN_MATERIAL_DISPLACEMENT_OVERRIDE_KEY,
				UMaterialInstanceConstant::StaticClass(),
				"BasePropertyOverrides.bOverride_DisplacementScaling"
			);

			const float Magnitude = Displacement->Get()->GetNumberField(TEXT("magnitude"));
			MaterialInstanceFactoryNode->SetAttribute(MEGASCAN_MATERIAL_DISPLACEMENT_MAGNITUDE_KEY, Magnitude);
			MaterialInstanceFactoryNode->AddApplyAndFillDelegates<float>(
				MEGASCAN_MATERIAL_DISPLACEMENT_MAGNITUDE_KEY,
				UMaterialInstanceConstant::StaticClass(),
				"BasePropertyOverrides.DisplacementScaling.Magnitude"
			);

			const float Center = Displacement->Get()->GetNumberField(TEXT("center"));
			MaterialInstanceFactoryNode->SetAttribute(MEGASCAN_MATERIAL_DISPLACEMENT_CENTER_KEY, Center);
			MaterialInstanceFactoryNode->AddApplyAndFillDelegates<float>(
				MEGASCAN_MATERIAL_DISPLACEMENT_CENTER_KEY,
				UMaterialInstanceConstant::StaticClass(),
				"BasePropertyOverrides.DisplacementScaling.Center"
			);
		}
	}

	const TArray<TSharedPtr<FJsonValue>>* CustomParams;
	if (!MaterialParams->TryGetArrayField(TEXT("params"), CustomParams))
	{
		return;
	}

	for (const TSharedPtr<FJsonValue>& Param : *CustomParams)
	{
		if (Param->Type != EJson::Object)
			continue;

		const TSharedPtr<FJsonObject>& ParamObject = Param->AsObject();
		const FString Name                         = ParamObject->GetStringField(TEXT("Name"));
		const TSharedPtr<FJsonValue> Value         = ParamObject->TryGetField(TEXT("Value"));

		const FString ParameterName = UInterchangeShaderPortsAPI::MakeInputValueKey(Name);
		if (Value->Type == EJson::Boolean)
		{
			MaterialInstanceFactoryNode->AddBooleanAttribute(ParameterName, Value->AsBool());
		}
		else if (Value->Type == EJson::Number)
		{
			MaterialInstanceFactoryNode->AddFloatAttribute(ParameterName, Value->AsNumber());
		}
		else if (Value->Type == EJson::Array)
		{
			const TArray<TSharedPtr<FJsonValue>>& ArrayValue = Value->AsArray();
			if (ArrayValue.Num() >= 4)
			{
				MaterialInstanceFactoryNode->AddLinearColorAttribute(
					ParameterName,
					FLinearColor(
						ArrayValue[0]->AsNumber(),
						ArrayValue[1]->AsNumber(),
						ArrayValue[2]->AsNumber(),
						ArrayValue[3]->AsNumber()
					)
				);
			}
		}
	}
}

void UInterchangeMegascansPipeline::SetupMaterialParents(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, const FString& CustomType)
{
	switch (MegascanImportType)
	{
		case EMegascanImportType::Model3D:
		{
			if (CustomType.IsEmpty() || CustomType == "base")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::Base);
			else if (CustomType == "masked")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::BaseMasked);
			else if (CustomType == "transmission")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::BaseTransmission);
			else if (CustomType == "fuzz")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::BaseFuzz);
			else if (CustomType == "glass")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::Glass);
		}
		break;
		case EMegascanImportType::Surface:
		{
			if (CustomType.IsEmpty() || CustomType == "surface")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::Surface);
			else if (CustomType == "masked")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::SurfaceMasked);
			else if (CustomType == "transmission")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::SurfaceTransmission);
			else if (CustomType == "fuzz")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::SurfaceFuzz);
			else if (CustomType == "fabric" || CustomType == "fabric_opaque")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::Fabric);
			else if (CustomType == "fabric_masked")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::FabricMasked);
		}
		break;
		case EMegascanImportType::Decal:
		{
			if (CustomType.IsEmpty() || CustomType == "decal")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::Decal);
		}
		break;
		case EMegascanImportType::Plant:
		{
			if (CustomType.IsEmpty() || CustomType == "plant")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::Plant);
			else if (CustomType == "billboard")
				SetMegascanMaterialType(MaterialInstanceFactoryNode, EMegascanMaterialType::PlantBillboard);
		}
		break;

		default:
			break;
	}

	UpdateParentMaterial(MaterialInstanceFactoryNode);
}
