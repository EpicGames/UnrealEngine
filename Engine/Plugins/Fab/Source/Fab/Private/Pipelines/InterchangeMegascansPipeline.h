// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InterchangePipelineBase.h"

#include "Engine/DeveloperSettings.h"

#include "Runtime/Launch/Resources/Version.h"

#include "InterchangeMegascansPipeline.generated.h"

class FJsonValue;
class FJsonObject;

class UMaterialInterface;
class UMaterialInstanceConstant;

class UInterchangeMeshNode;
class UInterchangeSceneNode;
class UInterchangeSourceData;
class UInterchangeMeshFactoryNode;
class UInterchangeBaseNodeContainer;
class UInterchangeTextureFactoryNode;
class UInterchangeStaticMeshFactoryNode;
class UInterchangeMaterialInstanceFactoryNode;
class UInterchangeGenericCommonMeshesProperties;

UENUM()
enum class EMegascanImportType
{
	Model3D UMETA(DisplayName = "3D"),
	Surface UMETA(DisplayName = "Surface"),
	Decal UMETA(DisplayName = "Decal"),
	Imperfection UMETA(DisplayName = "Imperfection"),
	Plant UMETA(DisplayName = "Plant"),
};

UENUM()
enum class EMegascanImportTier: int8
{
	Invalid = -1 UMETA(Hidden),
	Raw     = 0 UMETA(DisplayName = "Raw"),
	High    = 1 UMETA(DisplayName = "High"),
	Medium  = 2 UMETA(DisplayName = "Medium"),
	Low     = 3 UMETA(DisplayName = "Low"),
};

UENUM()
enum class EMegascanMaterialType : int32
{
	Invalid = 0 UMETA(Hidden),

	Base UMETA(DisplayName = "3D"),
	BaseMasked UMETA(DisplayName = "3D Masked"),
	BaseFuzz UMETA(DisplayName="3D Fuzz"),
	BaseTransmission UMETA(DisplayName="3D Transmission"),

	Glass UMETA(DisplayName="Glass"),

	Surface UMETA(DisplayName = "Surface"),
	SurfaceMasked UMETA(DisplayName = "Surface Masked"),
	SurfaceFuzz UMETA(DisplayName="Surface Fuzz"),
	SurfaceTransmission UMETA(DisplayName="Surface Transmission"),

	Fabric UMETA(DisplayName="Fabric"),
	FabricMasked UMETA(DisplayName="Fabric Masked"),

	Decal UMETA(DisplayName = "Decal"),

	Plant UMETA(DisplayName = "Plant"),
	PlantBillboard UMETA(DisplayName = "Plant Billboard"),
};

USTRUCT()
struct FMegascanMaterialPair
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Config, Category = "Standard Material", meta = (DisplayName = "Material"))
	TSoftObjectPtr<UMaterialInterface> StandardMaterial;

	UPROPERTY(EditAnywhere, Config, Category = "Virtual Material", meta = (DisplayName = "Material (with VT support)"))
	TSoftObjectPtr<UMaterialInterface> VTMaterial;
};

UCLASS(Config=Fab, meta = (DisplayName = "Fab Megascans"))
class UMegascansMaterialParentSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, config, Category = "Parent Materials", meta = (DisplayName = "Megascan Parent Materials"))
	TMap<EMegascanMaterialType, FMegascanMaterialPair> MaterialParents;

	// Version of the plugin materials that were last copied into this project.
	// Used to detect breaking changes when the plugin is updated.
	UPROPERTY(config)
	FString CopiedMaterialVersion;
};

UCLASS(BlueprintType)
class UInterchangeMegascansPipeline : public UInterchangePipelineBase
{
	GENERATED_BODY()

	UInterchangeMegascansPipeline();

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Megascans", meta = (DisplayName = "Megascan Asset Import Type"))
	EMegascanImportType MegascanImportType;

	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category="Megascans", meta = (DisplayName = "Megascan Import Settings"))
	TObjectPtr<UMegascansMaterialParentSettings> MegascansMaterialParentSettings;

	void SetCommonMeshesProperties(TWeakObjectPtr<UInterchangeGenericCommonMeshesProperties> InCommonMeshesProperties) { CommonMeshesProperties = InCommonMeshesProperties; }

protected:
	virtual void ExecutePipeline(
		UInterchangeBaseNodeContainer* NodeContainer,
		const TArray<UInterchangeSourceData*>& SourceDatas
		#if ENGINE_MAJOR_VERSION == 5 && ENGINE_MINOR_VERSION > 3
		,
		const FString& ContentBasePath
		#endif
	) override;

	virtual void ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* NodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport) override;
	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask) override { return true; }

private:
	bool LoadGltfSource(const FString& SourceFile);
	void ForEachGltfMaterial(const TFunction<void(const FString&, const TSharedPtr<FJsonObject>&)>& Callback) const;
	void ForEachGltfTexture(const TFunction<void(const FString&, const TSharedPtr<FJsonObject>&)>& Callback);
	void ForEachGltfMesh(const TFunction<void(const FString&, const TSharedPtr<FJsonObject>&)>& Callback);

	UInterchangeTextureFactoryNode* FindTextureFactoryNodeByName(const FString& DisplayName) const;
	UInterchangeStaticMeshFactoryNode* FindStaticMeshFactoryNodeByName(const FString& DisplayName) const;
	UInterchangeMaterialInstanceFactoryNode* FindMaterialInstanceFactoryNodeByName(const FString& DisplayName) const;

	const TSharedPtr<FJsonObject>* GetMaterialAtIndex(uint32 Index) const;

	EMegascanMaterialType GetMegascanMaterialType(const UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode) const;
	bool SetMegascanMaterialType(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, EMegascanMaterialType MaterialType) const;
	bool UpdateParentMaterial(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, bool bVTMaterial = false, bool bUpdateReferencedObject = false);

	FString GetStaticMeshLodDataNodeUid(const UInterchangeMeshFactoryNode* MeshFactoryNode, int32 LodIndex);
	FString GetStaticMeshLodDataNodeDisplayName(int32 LodIndex);

	void SetupMeshLod(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const UInterchangeSceneNode* MeshNode, int32 LodIndex);

	void SetFoliageType(const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode);

	void SetupTextureParams(UInterchangeTextureFactoryNode* TextureFactoryNode, const TSharedPtr<FJsonObject>& TextureParams);
	void SetupStaticMesh(UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode);
	void SetupStaticMeshParams(const UInterchangeStaticMeshFactoryNode* StaticMeshFactoryNode, const TSharedPtr<FJsonObject>& MeshParams);
	void SetupMaterial(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode);
	void SetupMaterialParams(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, const TSharedPtr<FJsonObject>& MaterialParams);

	void SetupMaterialParents(UInterchangeMaterialInstanceFactoryNode* MaterialInstanceFactoryNode, const FString& CustomType = "");

	template <class NodeType>
	TArray<NodeType*> GetNodesOfType()
	{
		TArray<NodeType*> Nodes;
		BaseNodeContainer->IterateNodesOfType<NodeType>(
			[&Nodes](const FString& Uid, NodeType* Node)
			{
				Nodes.Add(Node);
			}
		);
		return Nodes;
	}

	template <class NodeType>
	NodeType* FindNodeOfTypeByName(const FString& DisplayName)
	{
		NodeType* FoundNode = nullptr;
		BaseNodeContainer->IterateNodesOfType<NodeType>(
			[&](const FString& Uid, NodeType* Node)
			{
				if (FoundNode)
					return;
				if (Node->GetDisplayLabel() == DisplayName)
					FoundNode = Node;
			}
		);
		return FoundNode;
	}

private:
	UPROPERTY()
	TObjectPtr<UInterchangeBaseNodeContainer> BaseNodeContainer = nullptr;

	UPROPERTY(Transient)
	TWeakObjectPtr<UInterchangeGenericCommonMeshesProperties> CommonMeshesProperties;

	EMegascanImportTier MegascanAssetTier = EMegascanImportTier::Invalid;
	bool bVirtualTexturesImported         = false;

private:
	TSharedPtr<FJsonObject> GltfJson = nullptr;

	TArray<UInterchangeTextureFactoryNode*> TextureFactoryNodes;
	TArray<UInterchangeStaticMeshFactoryNode*> StaticMeshFactoryNodes;
	TArray<UInterchangeMaterialInstanceFactoryNode*> MaterialInstanceFactoryNodes;
};
