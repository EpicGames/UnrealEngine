// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeInstancedFoliageTypeFactoryNode.h"

#include "FoliageType_InstancedStaticMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeInstancedFoliageTypeFactoryNode)

FString UInterchangeInstancedFoliageTypeFactoryNode::GetNodeUidFromStaticMeshFactoryUid(const FString& StaticMeshFactoryUid)
{
	return UInterchangeFactoryBaseNode::BuildFactoryNodeUid(TEXT("InstancedFoliageType") + StaticMeshFactoryUid);
}

UClass* UInterchangeInstancedFoliageTypeFactoryNode::GetObjectClass() const
{
	return UFoliageType_InstancedStaticMesh::StaticClass();
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomStaticMesh(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(StaticMesh, FString);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomStaticMesh(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(StaticMesh, FString);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomScaling(EFoliageScaling& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(Scaling, EFoliageScaling);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomScaling(const EFoliageScaling AttributeValue, const bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeInstancedFoliageTypeFactoryNode, Scaling, EFoliageScaling, UFoliageType_InstancedStaticMesh);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomScaleX(FVector2f& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ScaleX, FVector2f);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomScaleX(const FVector2f& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ScaleX, FVector2f);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomScaleY(FVector2f& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ScaleY, FVector2f);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomScaleY(const FVector2f& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ScaleY, FVector2f);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomScaleZ(FVector2f& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ScaleZ, FVector2f);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomScaleZ(const FVector2f& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ScaleZ, FVector2f);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomAlignToNormal(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AlignToNormal, bool);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomAlignToNormal(const bool AttributeValue, const bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeInstancedFoliageTypeFactoryNode, AlignToNormal, bool, UFoliageType_InstancedStaticMesh);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomRandomYaw(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(RandomYaw, bool);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomRandomYaw(const bool AttributeValue, const bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeInstancedFoliageTypeFactoryNode, RandomYaw, bool, UFoliageType_InstancedStaticMesh);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomRandomPitchAngle(float& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(RandomPitchAngle, float);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomRandomPitchAngle(const float AttributeValue, const bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeInstancedFoliageTypeFactoryNode, RandomPitchAngle, float, UFoliageType_InstancedStaticMesh);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomAffectDistanceFieldLighting(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(bAffectDistanceFieldLighting, bool);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomAffectDistanceFieldLighting(const bool AttributeValue, const bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeInstancedFoliageTypeFactoryNode, bAffectDistanceFieldLighting, bool, UFoliageType_InstancedStaticMesh);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::GetCustomWorldPositionOffsetDisableDistance(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(WorldPositionOffsetDisableDistance, int32);
}

bool UInterchangeInstancedFoliageTypeFactoryNode::SetCustomWorldPositionOffsetDisableDistance(const int32 AttributeValue, const bool bAddApplyDelegate)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER(UInterchangeInstancedFoliageTypeFactoryNode, WorldPositionOffsetDisableDistance, int32, UFoliageType_InstancedStaticMesh);
}
