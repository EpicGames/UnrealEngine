// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Nodes/InterchangeFactoryBaseNode.h"

#include "InterchangeInstancedFoliageTypeFactoryNode.generated.h"

UCLASS(BlueprintType)
class UInterchangeInstancedFoliageTypeFactoryNode : public UInterchangeFactoryBaseNode
{
	GENERATED_BODY()

public:
	static FString GetNodeUidFromStaticMeshFactoryUid(const FString& StaticMeshFactoryUid);

public:
	virtual UClass* GetObjectClass() const override;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomStaticMesh(FString& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomStaticMesh(const FString& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomScaling(EFoliageScaling& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomScaling(const EFoliageScaling AttributeValue, const bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomScaleX(FVector2f& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomScaleX(const FVector2f& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomScaleY(FVector2f& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomScaleY(const FVector2f& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomScaleZ(FVector2f& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomScaleZ(const FVector2f& AttributeValue);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomAlignToNormal(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomAlignToNormal(const bool AttributeValue, const bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomRandomYaw(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomRandomYaw(const bool AttributeValue, const bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomRandomPitchAngle(float& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomRandomPitchAngle(const float AttributeValue, const bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomAffectDistanceFieldLighting(bool& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomAffectDistanceFieldLighting(const bool AttributeValue, const bool bAddApplyDelegate = true);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool GetCustomWorldPositionOffsetDisableDistance(int32& AttributeValue) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | InstancedFoliageTypeFactory")
	bool SetCustomWorldPositionOffsetDisableDistance(const int32 AttributeValue, const bool bAddApplyDelegate = true);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(StaticMesh);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(Scaling);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ScaleX);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ScaleY);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ScaleZ);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AlignToNormal);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RandomYaw);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(RandomPitchAngle);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(bAffectDistanceFieldLighting);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(WorldPositionOffsetDisableDistance);
};
