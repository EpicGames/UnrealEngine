// Copyright Epic Games, Inc. All Rights Reserved.

#include "FabContentSource.h"

#include "FabBrowser.h"
#include "FabMyFolderIntegration.h"
#include "IContentBrowserSingleton.h"
#include "Elements/Columns/TypedElementWebColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#define LOCTEXT_NAMESPACE "FabContentSource"

namespace FabContentSource::Private
{
	static FName ContentSourceName("FabMyLibraryContentSource");
	
	static bool bEnableContentSource = false;
	
	static FAutoConsoleVariableRef CVarEnableTestContentSource(
		TEXT("Fab.TEDS.MyLibrary.ContentSource"),
		bEnableContentSource,
		TEXT("Add a Content Source that displays your fab library in the Content Browser ")
		, FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*CVar*/)
		{
			if(bEnableContentSource)
			{
				IContentBrowserSingleton::Get().RegisterContentSourceFactory(ContentSourceName,
					IContentBrowserSingleton::FContentSourceFactory::CreateLambda(
					[]() -> TSharedRef<UE::Editor::ContentBrowser::IContentSource>
					{
						return MakeShared<FFabMyLibraryContentSource>();
					}));
			}
			else
			{
				IContentBrowserSingleton::Get().UnregisterContentSourceFactory(ContentSourceName);
			}
		}));
}

FName FFabMyLibraryContentSource::GetName()
{
	return FabContentSource::Private::ContentSourceName;
}

FText FFabMyLibraryContentSource::GetDisplayName()
{
	return LOCTEXT("MyLibraryContentSourceDisplayName", "Fab");
}

FSlateIcon FFabMyLibraryContentSource::GetIcon()
{
	return FSlateIcon(FFabBrowser::GetStyleSet().GetStyleSetName(), "Fab.ToolbarIcon");
}

void FFabMyLibraryContentSource::GetAssetViewInitParams(UE::Editor::ContentBrowser::FTableViewerInitParams& OutInitParams)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	OutInitParams.QueryDescription = Select()
			.Where()
				.All<FFabObjectColumn>()
			.Compile();

	OutInitParams.Columns = { FFabObjectNameColumn::StaticStruct(), FUrlColumn::StaticStruct() };

	OutInitParams.CellWidgetPurpose = IUiProvider::FPurposeID(IUiProvider::FPurposeInfo(
		"General", "Cell", NAME_None).GeneratePurposeID());

}

#undef LOCTEXT_NAMESPACE