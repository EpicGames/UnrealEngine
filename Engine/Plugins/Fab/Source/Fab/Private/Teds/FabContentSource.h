// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ContentSources/IContentSource.h"

class FFabMyLibraryContentSource : public UE::Editor::ContentBrowser::IContentSource
{
public:
	virtual ~FFabMyLibraryContentSource() override = default;

	virtual FName GetName() override;
	virtual FText GetDisplayName() override;
	virtual FSlateIcon GetIcon() override;
	virtual void GetAssetViewInitParams(UE::Editor::ContentBrowser::FTableViewerInitParams& OutInitParams) override;
};