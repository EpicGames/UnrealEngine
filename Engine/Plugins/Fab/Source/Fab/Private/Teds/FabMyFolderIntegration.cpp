// Copyright Epic Games, Inc. All Rights Reserved.

#include "Teds/FabMyFolderIntegration.h"

#include "DataStorage/Features.h"
#include "Dom/JsonObject.h"
#include "Elements/Columns/TypedElementWebColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "EOSShared.h"
#include "FabAuthentication.h"
#include "FabLog.h"
#include "FabSettings.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FabMyFolderIntegration)

void UFabFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Table = DataStorage.RegisterTable<FFabObjectNameColumn, FFabObjectColumn, FEditorDataStorageUrlColumn>(FName("Fab"));
}

void FFabTedsMyFolderIntegration::QueueSyncRequest()
{
	QueueSyncRequest(1000);
}

void FFabTedsMyFolderIntegration::QueueSyncRequest(uint32 BatchSize)
{
	using namespace UE::Editor::DataStorage;

	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		Storage->RemoveAllRowsWith<FFabObjectColumn>();

		QueueSyncRequest({}, BatchSize);
	}
}

void FFabTedsMyFolderIntegration::QueueSyncRequest(const FString& Cursor, uint32 BatchSize)
{
	if (EOS_Auth_GetLoggedInAccountsCount(FabAuthentication::AuthHandle) > 0)
	{
		const UFabSettings* Settings = GetDefault<UFabSettings>();

		EOS_Auth_Token* UserAuthToken = nullptr;

		EOS_Auth_CopyUserAuthTokenOptions CopyTokenOptions = { 0 };
		CopyTokenOptions.ApiVersion = EOS_AUTH_COPYUSERAUTHTOKEN_API_LATEST;

		if (EOS_Auth_CopyUserAuthToken(FabAuthentication::AuthHandle,
			&CopyTokenOptions, FabAuthentication::EpicAccountId, &UserAuthToken) == EOS_EResult::EOS_Success)
		{
			FString CursorString;
			if (!Cursor.IsEmpty())
			{
				CursorString = "&cursor=\"";
				CursorString += Cursor;
				CursorString += '"';
			}

			TSharedRef<IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
			HttpRequest->SetVerb("GET");
			HttpRequest->SetHeader("accept", "application/json");
			HttpRequest->SetHeader("Authorization", FString::Printf(TEXT("Bearer %s"), *FString(UserAuthToken->AccessToken)));
			HttpRequest->SetURL(FString::Printf(TEXT("%s/e/accounts/%s/ue/library?count=%u%s"),
				*Settings->GetUrlFromEnvironment(),
				*LexToString(FabAuthentication::EpicAccountId), BatchSize, *CursorString));
			HttpRequest->OnProcessRequestComplete().BindLambda([BatchSize](HttpRequestPtr Request, HttpResponsePtr Response, bool bWasSuccessful)
				{
					ProcessSyncResults(BatchSize, Request, Response, bWasSuccessful);
				});
			HttpRequest->ProcessRequest();

			EOS_Auth_Token_Release(UserAuthToken);
		}
	}
	else
	{
		FAB_LOG("Unable to retrieve My Folder data due to user not being logged into Fab.");
	}
}

void FFabTedsMyFolderIntegration::ProcessSyncResults(uint32 BatchSize, HttpRequestPtr Request, HttpResponsePtr Response, bool bWasSuccessful)
{
	using namespace UE::Editor::DataStorage;

	FAB_LOG("Result for (portion of) the My Folder data.");
	
	FString ContentType = bWasSuccessful ? Response->GetContentType() : FString();
	if (bWasSuccessful && Response->GetContentType().StartsWith(TEXT("application/json")))
	{
		FString Content = Response->GetContentAsString();
		TSharedPtr<FJsonObject> JsonResults = MakeShareable(new FJsonObject());
		TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(Content);
		if (FJsonSerializer::Deserialize(JsonReader, JsonResults))
		{
			if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				if (const UFabFactory* FabFactory = Storage->FindFactory<UFabFactory>())
				{
					// Determine if this is the last set or that there are more results to retrieve.
					const TSharedPtr<FJsonObject>* Cursors = nullptr;
					if (JsonResults->TryGetObjectField(TEXT("cursors"), Cursors))
					{
						FString Next;
						if ((*Cursors)->TryGetStringField(TEXT("next"), Next))
						{
							QueueSyncRequest(Next, BatchSize);
						}
					}

					// Process the returned results.
					const TArray<TSharedPtr<FJsonValue>>* Results = nullptr;
					if (JsonResults->TryGetArrayField(TEXT("results"), Results); Results && !Results->IsEmpty())
					{
						FString Temp;

						TArray<RowHandle> FailedSerializationRows;

						int32 Index = 0;
						Storage->BatchAddRow(FabFactory->Table, Results->Num(),
							[Storage, Results = *Results, &Index, &Temp, &FailedSerializationRows](RowHandle Row)
							{
								const TSharedPtr<FJsonValue>& Result = Results[Index++];
								
								const TSharedPtr<FJsonObject>* ResultObject = nullptr;
								if (bool Found = Result->TryGetObject(ResultObject); Found && ResultObject->IsValid())
								{
									if (FFabObjectNameColumn* Target = Storage->GetColumn<FFabObjectNameColumn>(Row))
									{
										SetNameColumn(*Target, **ResultObject, Temp);
									}

									if (FFabObjectColumn* Target = Storage->GetColumn<FFabObjectColumn>(Row))
									{
										SetFabObjectColumn(*Target, **ResultObject, Temp);
									}

									if (FUrlColumn* Target = Storage->GetColumn<FUrlColumn>(Row))
									{
										SetUrlColumn(*Target, **ResultObject, Temp);
									}

									AddDistributionMethod(Storage, Row, **ResultObject, Temp);
									AddImages(Storage, Row, **ResultObject, Temp);
								}
								else
								{
									FailedSerializationRows.Add(Row);
								}
							});

						if (!FailedSerializationRows.IsEmpty())
						{
							Storage->BatchRemoveRows(FailedSerializationRows);
						}

						FAB_LOG("Parsed data for (portion of) My Folder.");
					}
					else
					{
						FAB_LOG_ERROR("Unable to store My Folder data due missing results. An error may have occurred: %s", *Content);
					}
				}
				else
				{
					FAB_LOG_ERROR("Unable to store My Folder data due the factory for Fab objects hasn't been initialized.");
				}
			}
			else
			{
				FAB_LOG_ERROR("Unable to store My Folder data due the editor data storage not being available.");
			}
		}
		else
		{
			FAB_LOG_ERROR("Unable to retrieve My Folder data due to returned result not being valid JSON.");
		}
	}
	else
	{
		FAB_LOG_ERROR("Unable to retrieve My Folder data due to the request failing or not returning a JSON document with the required data.");
	}
}

void FFabTedsMyFolderIntegration::SetNameColumn(FFabObjectNameColumn& Target, const FJsonObject& Object, FString& Temp)
{
	if (Object.TryGetStringField(TEXT("title"), Temp))
	{
		Target.Name = FName(Temp);
		Temp.Reset();
	}
}

void FFabTedsMyFolderIntegration::SetFabObjectColumn(FFabObjectColumn& Target, const FJsonObject& Object, FString& Temp)
{
	Object.TryGetStringField(TEXT("description"), Target.Description);

	if (Object.TryGetStringField(TEXT("assetId"), Temp))
	{
		FGuid::Parse(Temp, Target.AssetId);
		Temp.Reset();
	}
	if (Object.TryGetStringField(TEXT("assetNamespace"), Temp))
	{
		FGuid::Parse(Temp, Target.AssetNamespace);
		Temp.Reset();
	}

	if (Object.TryGetStringField(TEXT("listingType"), Temp))
	{
		Target.ListingType = FName(Temp);
		Temp.Reset();
	}

	Object.TryGetStringField(TEXT("seller"), Target.Seller);

	if (Object.TryGetStringField(TEXT("source"), Temp))
	{
		Target.Source = FName(Temp);
		Temp.Reset();
	}

	/* Additionally available data :
	* projectVersions": 
	  [
        {
          "artifactId": {},
          "engineVersions": [],
          "targetPlatforms": [],
		  "buildVersions": []
        }
      ],
	  "customAttributes": []
	  "legacyItemId": {}
	*/
}

void FFabTedsMyFolderIntegration::SetUrlColumn(UE::Editor::DataStorage::FUrlColumn& Target, const FJsonObject& Object, FString& Temp)
{
	Object.TryGetStringField(TEXT("url"), Target.UrlString);
}

void FFabTedsMyFolderIntegration::AddDistributionMethod(
	UE::Editor::DataStorage::ICoreProvider* Storage, UE::Editor::DataStorage::RowHandle Row, const FJsonObject& Object, FString& Temp)
{
	if (Object.TryGetStringField(TEXT("distributionMethod"), Temp))
	{
		Temp.ToLowerInline();
		Temp.ReplaceCharInline(TEXT('_'), TEXT(' '));

		Storage->AddColumn<FFabDistributionMethodTag>(Row, FName(Temp));
		Temp.Reset();
	}
}

void FFabTedsMyFolderIntegration::AddImages(
	UE::Editor::DataStorage::ICoreProvider* Storage, UE::Editor::DataStorage::RowHandle Row, const FJsonObject& Object, FString& Temp)
{
	using namespace UE::Editor::DataStorage;

	const TArray<TSharedPtr<FJsonValue>>* Images = nullptr;
	if (Object.TryGetArrayField(TEXT("images"), Images); Images && !Images->IsEmpty())
	{
		for (const TSharedPtr<FJsonValue>& Image : *Images)
		{
			const TSharedPtr<FJsonObject>* ImageObject = nullptr;
			if (bool Found = Image->TryGetObject(ImageObject); Found && ImageObject->IsValid())
			{
				FWebImageColumn ImageColumn;
				(*ImageObject)->TryGetStringField(TEXT("url"), ImageColumn.UrlString);
				(*ImageObject)->TryGetNumberField(TEXT("width"), ImageColumn.Width);
				(*ImageObject)->TryGetNumberField(TEXT("height"), ImageColumn.Height);
				
				(*ImageObject)->TryGetStringField(TEXT("type"), Temp);
				Temp.TrimStartAndEndInline();
				Storage->AddColumn(Row, FName(Temp), MoveTemp(ImageColumn));
			}
		}
	}
}
