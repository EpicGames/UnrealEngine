// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Templates/SharedPointer.h"

#include "FabMyFolderIntegration.generated.h"

struct FEditorDataStorageUrlColumn;
class FJsonObject;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

UCLASS()
class UFabFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UFabFactory() override = default;

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	UE::Editor::DataStorage::TableHandle Table;
};

USTRUCT(meta = (DisplayName = "Fab distribution method", EditorDataStorage_DynamicColumnTemplate))
struct FFabDistributionMethodTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Name"))
struct FFabObjectNameColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FName Name;
};

USTRUCT(meta = (DisplayName = "Fab object"))
struct FFabObjectColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable))
	FString Description;
	
	UPROPERTY()
	FGuid AssetId;
	
	UPROPERTY()
	FGuid AssetNamespace;

	UPROPERTY(meta = (Searchable))
	FName ListingType;

	UPROPERTY(meta = (Searchable))
	FString Seller;

	UPROPERTY()
	FName Source;

	UPROPERTY()
	FString UrlString;
};

class FFabTedsMyFolderIntegration
{
public:
	static void QueueSyncRequest();
	static void QueueSyncRequest(uint32 BatchSize);

private:
	using HttpRequestPtr = TSharedPtr<class IHttpRequest, ESPMode::ThreadSafe>;
	using HttpResponsePtr = TSharedPtr<class IHttpResponse, ESPMode::ThreadSafe>;

	static void QueueSyncRequest(const FString& Cursor, uint32 BatchSize);
	static void ProcessSyncResults(uint32 BatchSize, HttpRequestPtr Request, HttpResponsePtr Response, bool bWasSuccessful);

	static void SetNameColumn(FFabObjectNameColumn& Target, const FJsonObject& Object, FString& Temp);
	static void SetFabObjectColumn(FFabObjectColumn& Target, const FJsonObject& Object, FString& Temp);
	static void SetUrlColumn(FEditorDataStorageUrlColumn& Target, const FJsonObject& Object, FString& Temp);
	static void AddDistributionMethod(
		UE::Editor::DataStorage::ICoreProvider* Storage, UE::Editor::DataStorage::RowHandle Row, const FJsonObject& Object, FString& Temp);
	static void AddImages(
		UE::Editor::DataStorage::ICoreProvider* Storage, UE::Editor::DataStorage::RowHandle Row, const FJsonObject& Object, FString& Temp);
};