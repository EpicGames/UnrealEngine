// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/AssetUtils.h"

#include "FabLog.h"
#include "IContentBrowserSingleton.h"

#include "AssetRegistry/IAssetRegistry.h"

#include "HAL/PlatformFileManager.h"

#include "Misc/FileHelper.h"

#include "FileUtilities/ZipArchiveReader.h"

void FAssetUtils::SanitizeFolderName(FString& AssetID)
{
	const TCHAR* InvalidChar = INVALID_OBJECTPATH_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	while (*InvalidChar)
	{
		AssetID.ReplaceCharInline(*InvalidChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidChar;
	}

	AssetID.ReplaceCharInline('/', TCHAR('_'), ESearchCase::CaseSensitive);
}

void FAssetUtils::SanitizePath(FString& Path)
{
	const TCHAR* InvalidChar = INVALID_OBJECTPATH_CHARACTERS INVALID_LONGPACKAGE_CHARACTERS;

	while (*InvalidChar)
	{
		Path.ReplaceCharInline(*InvalidChar, TCHAR('_'), ESearchCase::CaseSensitive);
		++InvalidChar;
	}
}

bool FAssetUtils::Unzip(const FString& Path, const FString& TargetPath)
{
	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();

	IFileHandle* ArchiveFileHandle = FileManager.OpenRead(*Path);
	const FZipArchiveReader ZipArchiveReader(ArchiveFileHandle);
	if (!ZipArchiveReader.IsValid())
	{
		FAB_LOG_ERROR("Error opening archive file");
		return false;
	}

	const TArray<FString> ArchiveFiles = ZipArchiveReader.GetFileNames();
	for (const FString& FileName : ArchiveFiles)
	{
		if (FileName.EndsWith("/") || FileName.EndsWith("\\"))
			continue;
		const FString AbsoluteDestFileName = TargetPath / FileName;
		if (!FPaths::IsUnderDirectory(AbsoluteDestFileName, TargetPath))
			continue;
		if (TArray<uint8> FileBuffer; ZipArchiveReader.TryReadFile(FileName, FileBuffer))
		{
			if (!FFileHelper::SaveArrayToFile(FileBuffer, *(TargetPath / FileName)))
			{
				FAB_LOG_ERROR("Error saving unarchived data to file");
			}
		}
	}

	return true;
}

void FAssetUtils::ScanForAssets(const FString& FolderPath)
{
	IAssetRegistry::Get()->ScanPathsSynchronous(
		{
			FolderPath
		},
		true
	);
}

void FAssetUtils::SyncContentBrowserToFolder(const FString& FolderPath, const bool bFocusContentBrowser)
{
	IContentBrowserSingleton::Get().SyncBrowserToFolders(
		{
			FolderPath
		},
		false,
		bFocusContentBrowser
	);
}

void FAssetUtils::SyncContentBrowserToFolders(const TArray<FString>& Folders, const bool bFocusContentBrowser)
{
	IContentBrowserSingleton::Get().SyncBrowserToFolders(Folders, false, bFocusContentBrowser);
}
