// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragImportOperation.h"

#include "FabBrowser.h"
#include "FabLog.h"
#include "IWebBrowserWindow.h"

#include "Animation/SkeletalMeshActor.h"
#include "Components/DecalComponent.h"

#include "Components/StaticMeshComponent.h"
#include "Components/SkeletalMeshComponent.h"

#include "Engine/DecalActor.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/SkeletalMesh.h"

#include "Framework/Application/SlateApplication.h"

#include "Importers/FabDragDropOp.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"

FDragImportOperation::FDragImportOperation(const UObject* InDraggedObject, EDragAssetType InDragAssetType)
	: DraggedAsset(InDraggedObject)
	, DragAssetType(InDragAssetType)
	, DragDropState(EDragDropState::Dragging)
{
	if (InDraggedObject == nullptr)
	{
		DraggedAsset  = FAssetData(FSoftObjectPath("/Fab/Placeholders/MeshPlaceholder.MeshPlaceholder").TryLoad());
		DragAssetType = EDragAssetType::Mesh;
	}
	InitializeDrag();
}

FDragImportOperation::FDragImportOperation(const FAssetData& InDraggedObject, EDragAssetType InDragAssetType)
	: DraggedAsset(InDraggedObject)
	, DragAssetType(InDragAssetType)
{
	InitializeDrag();
}

FDragImportOperation::~FDragImportOperation()
{
	if (DragOperationHandle)
	{
		DragOperationHandle->Cancel();
	}
}

void FDragImportOperation::InitializeDrag()
{
	CancelOperation();

	DragDropState       = EDragDropState::Dragging;
	DragOperationHandle = FFabDragDropOp::New(DraggedAsset, DragAssetType);
	DragOperationHandle->OnDrop().BindLambda([this]() { DragDropState = EDragDropState::Dropped; });

	FPointerEvent FakePointerEvent(
		FSlateApplication::Get().GetUserIndexForMouse(),
		FSlateApplicationBase::CursorPointerIndex,
		FSlateApplication::Get().GetCursorPos(),
		FSlateApplication::Get().GetLastCursorPos(),
		{
			EKeys::LeftMouseButton
		},
		EKeys::Invalid,
		0,
		{}
	);

	FDragDropEvent DragDropEvent(FakePointerEvent, DragOperationHandle);

	FFabBrowserTabContext* const& FabBrowserTabContext = FFabBrowser::GetActiveDockTabContext();

	if (FabBrowserTabContext && FabBrowserTabContext->WebBrowserWindow.IsValid())
	{
		TSharedPtr<SWindow> ParentWindow = FabBrowserTabContext->WebBrowserWindow->GetParentWindow();
		FSlateApplication::Get().ProcessDragEnterEvent(ParentWindow.ToSharedRef(), DragDropEvent);

		// In native (non-OSR) rendering mode, CEF owns a real child HWND and holds OS
		// mouse capture from the original WM_LBUTTONDOWN. That means WM_LBUTTONUP is
		// delivered to CEF rather than Slate, so Slate never sees the button release and
		// stays in drag-drop mode until the next unrelated click.
		// Transferring capture to the parent Slate window here ensures WM_LBUTTONUP
		// goes through Slate's own input pipeline and ends the drag naturally.
		if (FabBrowserTabContext->bIsNativelyRendered)
		{
			if (TSharedPtr<FGenericWindow> NativeWindow = ParentWindow->GetNativeWindow())
			{
				FSlateApplication::Get().GetPlatformApplication()->SetCapture(NativeWindow);
			}
		}
	}
}

void FDragImportOperation::UpdateDraggedAsset(UObject* InDraggedObject, const EDragAssetType InDragAssetType)
{
	check(InDraggedObject != nullptr);

	DraggedAsset  = FAssetData(InDraggedObject);
	DragAssetType = InDragAssetType;
	if (DragDropState == EDragDropState::Dragging)
	{
		InitializeDrag();
	}
	else
	{
		ReplaceSpawnedActor();
	}
}

void FDragImportOperation::UpdateDraggedAsset(const FAssetData& InDraggedObject, const EDragAssetType InDragAssetType)
{
	DraggedAsset  = InDraggedObject;
	DragAssetType = InDragAssetType;
	if (DragDropState == EDragDropState::Dragging)
	{
		InitializeDrag();
	}
	else
	{
		ReplaceSpawnedActor();
	}
}

void FDragImportOperation::CancelOperation()
{
	if (DragOperationHandle)
	{
		DragOperationHandle->Cancel();
		DragOperationHandle.Reset();
	}
	FSlateApplication::Get().CancelDragDrop();
}

TWeakObjectPtr<AActor> FDragImportOperation::GetSpawnedActor() const
{
	if (DragOperationHandle)
	{
		return DragOperationHandle->SpawnedActor;
	}

	return nullptr;
}

void FDragImportOperation::DeleteSpawnedActor() const
{
	if (DragOperationHandle)
	{
		DragOperationHandle->DestroySpawnedActor();
	}
}

void FDragImportOperation::ReplaceSpawnedActor() const
{
	if (GetSpawnedActor() == nullptr)
	{
		FAB_LOG_ERROR("No spawn actor found");
		return;
	}

	UObject* NewObject = DraggedAsset.GetAsset();
	bool bIsReplaced   = false;

	const TWeakObjectPtr<AActor>& SpawnedActor = GetSpawnedActor();
	if (!SpawnedActor.IsValid())
	{
		DragOperationHandle->DestroySpawnedActor();
		return;
	}

	if (DragAssetType == EDragAssetType::Mesh)
	{
		if (UStaticMesh* SourceMesh = Cast<UStaticMesh>(NewObject))
		{
			if (AStaticMeshActor* SMActor = Cast<AStaticMeshActor>(SpawnedActor))
			{
				if (SMActor->GetStaticMeshComponent())
				{
					SMActor->GetStaticMeshComponent()->EmptyOverrideMaterials();
					SMActor->GetStaticMeshComponent()->SetStaticMesh(SourceMesh);
					SMActor->SetActorLabel(DraggedAsset.AssetName.ToString());
					bIsReplaced = true;
				}
			}
		}
		else if (USkeletalMesh* SourceSkeletalMesh = Cast<USkeletalMesh>(NewObject))
		{
			ASkeletalMeshActor* SkMActor = Cast<ASkeletalMeshActor>(SpawnedActor);
			if (SkMActor == nullptr)
			{
				SkMActor             = SpawnedActor->GetWorld()->SpawnActor<ASkeletalMeshActor>(
					ASkeletalMeshActor::StaticClass(),
					SpawnedActor->GetTransform()
				);
				if (DragOperationHandle)
					DragOperationHandle->SpawnedActor = SkMActor;
				SpawnedActor->Destroy();
			}
			if (SkMActor->GetSkeletalMeshComponent())
			{
				SkMActor->GetSkeletalMeshComponent()->EmptyOverrideMaterials();
				SkMActor->GetSkeletalMeshComponent()->SetSkeletalMesh(SourceSkeletalMesh);
				SkMActor->SetActorLabel(DraggedAsset.AssetName.ToString());
				bIsReplaced = true;
			}
		}
	}
	else if (DragAssetType == EDragAssetType::Material)
	{
		UMaterialInterface* Material = Cast<UMaterialInterface>(NewObject);
		if (const AStaticMeshActor* SMActor = Cast<AStaticMeshActor>(SpawnedActor))
		{
			if (SMActor->GetStaticMeshComponent())
			{
				SMActor->GetStaticMeshComponent()->SetMaterial(0, Material);
				bIsReplaced = true;
			}
		}
		else if (const ASkeletalMeshActor* SkMActor = Cast<ASkeletalMeshActor>(SpawnedActor))
		{
			if (SkMActor->GetSkeletalMeshComponent())
			{
				SkMActor->GetSkeletalMeshComponent()->SetMaterial(0, Material);
				bIsReplaced = true;
			}
		}
	}
	else if (DragAssetType == EDragAssetType::Decal)
	{
		if (UMaterialInterface* Material = Cast<UMaterialInterface>(NewObject))
		{
			if (ADecalActor* DecalActor = Cast<ADecalActor>(SpawnedActor))
			{
				DecalActor->SetActorLabel(DraggedAsset.AssetName.ToString());
				DecalActor->SetDecalMaterial(Material);
				DecalActor->GetDecal()->PostEditChange();
				bIsReplaced = true;
			}
		}
	}

	if (!bIsReplaced && DragOperationHandle)
	{
		DragOperationHandle->DestroySpawnedActor();
	}
}
