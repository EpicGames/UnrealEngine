// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Importers/FabDragDropOp.h"

class FFabDragDropOp;

enum class EDragDropState
{
	Dragging,
	Dropped,
};

class FDragImportOperation
{
public:
	FDragImportOperation(const UObject* InDraggedObject = nullptr, EDragAssetType InDragAssetType = EDragAssetType::Mesh);
	FDragImportOperation(const FAssetData& InDraggedObject, EDragAssetType InDragAssetType = EDragAssetType::Mesh);

	~FDragImportOperation();

	void InitializeDrag();

	void UpdateDraggedAsset(UObject* InDraggedObject, const EDragAssetType InDragAssetType);
	void UpdateDraggedAsset(const FAssetData& InDraggedObject, const EDragAssetType InDragAssetType);

	void CancelOperation();

	TWeakObjectPtr<AActor> GetSpawnedActor() const;
	void DeleteSpawnedActor() const;
	void ReplaceSpawnedActor() const;

private:
	FAssetData DraggedAsset;
	EDragAssetType DragAssetType;

	EDragDropState DragDropState;

	TSharedPtr<FFabDragDropOp> DragOperationHandle;
};
