// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utilities/FabAssetsCache.h"

#include "FabSettings.h"

#include "HAL/PlatformProcess.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"

int64 FFabAssetsCache::CacheExpirationTimeoutInDays = 10;

FString SizeSuffix(const int64 InSize)
{
	static TArray SizeSuffixes = {
		TEXT("bytes"),
		TEXT("KiB"),
		TEXT("MiB"),
		TEXT("GiB"),
		TEXT("TiB")
	};
	if (InSize == 0)
		return FString::Printf(TEXT("%d bytes"), 0);
	const int64 Mag = FMath::Log2(static_cast<double>(InSize)) / 10.0;
	const double AdjSize = static_cast<double>(InSize) / (1ULL << (Mag * 10));
	return FString::Printf(TEXT("%.2f %s"), AdjSize, SizeSuffixes[Mag]);
}

FString FFabAssetsCache::GetCacheLocation()
{
	return GetDefault<UFabSettings>()->CacheDirectoryPath.Path;
}

TArray<FString> FFabAssetsCache::GetCachedAssets()
{
	TArray<FString> CachedAssets;
	IFileManager::Get().IterateDirectory(
		*GetCacheLocation(),
		[&CachedAssets](const TCHAR* Path, bool bIsDirectory)-> bool
		{
			if (!bIsDirectory && FPaths::GetExtension(Path) == "zip")
			{
				CachedAssets.Add(FPaths::GetBaseFilename(Path));
			}
			return true;
		}
	);
	return CachedAssets;
}

int64 FFabAssetsCache::GetCacheSize()
{
	int64 CacheSize = 0;
	IFileManager::Get().IterateDirectoryStatRecursively(
		*GetCacheLocation(),
		[&CacheSize](const TCHAR* Path, const FFileStatData& Stat)-> bool
		{
			if (!Stat.bIsDirectory)
			{
				CacheSize += Stat.FileSize;
			}
			return true;
		}
	);
	return CacheSize;
}

FText FFabAssetsCache::GetCacheSizeString()
{
	return FText::FromString(SizeSuffix(GetCacheSize()));
}

void FFabAssetsCache::ClearCache()
{
	IFileManager& FileManager = IFileManager::Get();
	FileManager.DeleteDirectory(*GetCacheLocation(), false, true);
	FileManager.MakeDirectory(*GetCacheLocation());
}

bool FFabAssetsCache::IsCached(const FString& AssetId, int64 DownloadSize)
{
	IFileManager& FileManager = IFileManager::Get();
	const FString CachedFile = GetCachedFile(AssetId);
	const FFileStatData CachedFileStats = FileManager.GetStatData(*CachedFile);

	return (CachedFileStats.bIsValid && CachedFileStats.FileSize == DownloadSize && (FDateTime::Now() - CachedFileStats.ModificationTime).
		GetTotalDays() <
		CacheExpirationTimeoutInDays);
}

FString FFabAssetsCache::GetCachedFile(const FString AssetId)
{
	return GetCacheLocation() / AssetId;
}

FString FFabAssetsCache::CacheAsset(const FString& DownloadedAssetPath)
{
	const FString CacheFilePath = GetCacheLocation() / FPaths::GetCleanFilename(DownloadedAssetPath);
	IFileManager::Get().Move(*CacheFilePath, *DownloadedAssetPath, true, true);
	return CacheFilePath;
}
