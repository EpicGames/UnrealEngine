// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

#include "AssetRegistry/IAssetRegistry.h"

#include "Async/Async.h"

#include "Misc/Paths.h"

#include "FabLocalAssets.generated.h"

UCLASS(config=EditorPerProjectUserSettings, MinimalAPI)
class UFabLocalAssets : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(config)
	TMap<FString, FString> PathsListingID;
	TMap<FString, FString> ListingIdPath;

public:
	static void AddLocalAsset(const FString& Path, const FString& AssetID)
	{
		UFabLocalAssets* LocalAssets = GetMutableDefault<UFabLocalAssets>();
		LocalAssets->PathsListingID.Add(Path, AssetID);
		LocalAssets->ListingIdPath.Add(AssetID, Path);
		Async(
			EAsyncExecution::TaskGraph,
			[]()
			{
				const IAssetRegistry* const AssetRegistry = IAssetRegistry::Get();

				UFabLocalAssets* LocalAssets = GetMutableDefault<UFabLocalAssets>();
				TArray<FString> ToRemoveKeys;
				for (auto [InPath, ListingID] : LocalAssets->PathsListingID)
				{
					if (!AssetRegistry->PathExists(InPath))
					{
						ToRemoveKeys.Add(InPath);
						LocalAssets->ListingIdPath.Remove(ListingID);
					}
				}
				for (const FString& RemoveKey : ToRemoveKeys)
				{
					LocalAssets->PathsListingID.Remove(RemoveKey);
				}
				LocalAssets->SaveConfig();
			}
		);
	}
	
	static const FString* FindPath(const FString& AssetID)
	{
		const UFabLocalAssets* LocalAssets = GetDefault<UFabLocalAssets>();
		return LocalAssets->ListingIdPath.Find(AssetID);
	}

	static const FString* FindListingID(const FString& Path)
	{
		const UFabLocalAssets* LocalAssets = GetDefault<UFabLocalAssets>();
		TArray<FString> PathParts;
		FPaths::GetPath(Path).ParseIntoArray(PathParts, TEXT("/"));
		FString PathBuilder = "/";
		for (const FString& PathPart : PathParts)
		{
			PathBuilder /= PathPart;
			if (const FString* FoundAssetID = LocalAssets->PathsListingID.Find(PathBuilder))
			{
				return FoundAssetID;
			}
		}
		return nullptr;
	}

	static void GetListingID(const FString& Path, FString& AssetID)
	{
		if (const FString* FoundAssetID = FindListingID(Path))
		{
			AssetID = *FoundAssetID;
		}
	}
};
