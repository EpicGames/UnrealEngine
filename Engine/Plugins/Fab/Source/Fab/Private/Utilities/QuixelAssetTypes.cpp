// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuixelAssetTypes.h"

#include "JsonObjectConverter.h"
#include "Misc/FileHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(QuixelAssetTypes)

TTuple<FString, FString> FQuixelAssetTypes::ExtractMeta(const FString& JsonFile, const FString& GltfFile)
{
	FString FileContent;
	FFileHelper::LoadFileToString(FileContent, *JsonFile);

	FAssetMetaDataJson AssetMetaDataJson;
	FJsonObjectConverter::JsonObjectStringToUStruct(FileContent, &AssetMetaDataJson);
	const FString AssetId = AssetMetaDataJson.Id;

	if (!GltfFile.IsEmpty() && AssetMetaDataJson.Displacement_Scale_Tier1 >= 0.0f && AssetMetaDataJson.Displacement_Bias_Tier1 >= 0.0f)
	{
		// Temporary till displacement values are integrated in gltf
		const TSharedPtr<FJsonObject> DisplacementObject = MakeShareable(new FJsonObject);
		DisplacementObject->SetNumberField(TEXT("magnitude"), AssetMetaDataJson.Displacement_Scale_Tier1);
		DisplacementObject->SetNumberField(TEXT("center"), AssetMetaDataJson.Displacement_Bias_Tier1);
		if (FString GltfFileData; FFileHelper::LoadFileToString(GltfFileData, *GltfFile))
		{
			TSharedPtr<FJsonObject> GltfJson = MakeShareable(new FJsonObject);
			if (FJsonSerializer::Deserialize(TJsonReaderFactory<TCHAR>::Create(GltfFileData), GltfJson))
			{
				if (const TArray<TSharedPtr<FJsonValue>>* Materials; GltfJson->TryGetArrayField(TEXT("materials"), Materials))
				{
					for (const TSharedPtr<FJsonValue>& Material : *Materials)
					{
						const TSharedPtr<FJsonObject>& MaterialObject = Material->AsObject();
						if (const TSharedPtr<FJsonObject>* Extras; MaterialObject->TryGetObjectField(TEXT("extras"), Extras))
						{
							if (const TSharedPtr<FJsonObject>* Overrides; Extras->Get()->TryGetObjectField(TEXT("overrides"), Overrides))
							{
								Overrides->Get()->SetObjectField(TEXT("displacement"), DisplacementObject);
							}
							else
							{
								TSharedPtr<FJsonObject> OverridesObject = MakeShareable(new FJsonObject);
								OverridesObject->SetObjectField(TEXT("displacement"), DisplacementObject);
								Extras->Get()->SetObjectField(TEXT("overrides"), OverridesObject);
							}
						}
						else
						{
							TSharedPtr<FJsonObject> OverridesObject = MakeShareable(new FJsonObject);
							OverridesObject->SetObjectField(TEXT("displacement"), DisplacementObject);
							TSharedPtr<FJsonObject> ExtrasObject = MakeShareable(new FJsonObject);
							ExtrasObject->SetObjectField(TEXT("overrides"), OverridesObject);

							MaterialObject->SetObjectField(TEXT("extras"), ExtrasObject);
						}
					}
				}

				if (FString SerializedJson; FJsonSerializer::Serialize(GltfJson.ToSharedRef(), TJsonWriterFactory<TCHAR>::Create(&SerializedJson, 2)))
				{
					FFileHelper::SaveStringToFile(SerializedJson, *GltfFile);
				}
			}
		}
	}

	if (AssetMetaDataJson.Categories.Num() == 0)
	{
		return {
			AssetId,
			""
		};
	}

	if (AssetMetaDataJson.Categories[0] == "3d")
	{
		return {
			AssetId,
			"3D"
		};
	}

	if (AssetMetaDataJson.Categories[0] == "surface")
	{
		return {
			AssetId,
			"Surfaces"
		};
	}

	if (AssetMetaDataJson.Categories[0] == "3dplant")
	{
		return {
			AssetId,
			"Plants"
		};
	}

	if (AssetMetaDataJson.Categories[0] == "atlas" && AssetMetaDataJson.Categories.Num() > 1)
	{
		if (AssetMetaDataJson.Categories[1] == "decals")
		{
			return {
				AssetId,
				"Decals"
			};
		}
		if (AssetMetaDataJson.Categories[1] == "imperfections")
		{
			return {
				AssetId,
				"Imperfections"
			};
		}
	}

	if (AssetMetaDataJson.SemanticTags.Asset_Type == "decal")
	{
		return {
			AssetId,
			"Decals"
		};
	}

	return {
		AssetId,
		""
	};
}
