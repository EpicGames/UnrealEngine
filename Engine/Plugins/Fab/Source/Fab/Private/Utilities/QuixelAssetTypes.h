// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "QuixelAssetTypes.generated.h"

USTRUCT()
struct FSemanticTags
{
	GENERATED_BODY()

	UPROPERTY()
	FString Asset_Type;
};

USTRUCT()
struct FAssetMetaDataJson
{
	GENERATED_BODY()

	UPROPERTY()
	FString Id;

	UPROPERTY()
	TArray<FString> Categories;

	UPROPERTY()
	FSemanticTags SemanticTags;

	UPROPERTY()
	float Displacement_Bias_Tier1 = -1.0f;

	UPROPERTY()
	float Displacement_Scale_Tier1 = -1.0f;
};

class FQuixelAssetTypes
{
public:
	static TTuple<FString, FString> ExtractMeta(const FString& JsonFile, const FString& GltfFile = "");
};
