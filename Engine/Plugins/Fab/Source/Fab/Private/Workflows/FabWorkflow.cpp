// Copyright Epic Games, Inc. All Rights Reserved.

#include "Workflows/FabWorkflow.h"

#include "NotificationProgressWidget.h"

#include "Framework/Notifications/NotificationManager.h"

#include "Widgets/Notifications/SNotificationList.h"

void IFabWorkflow::CreateDownloadNotification()
{
	// Create the notification info
	FNotificationInfo Info(FText::FromString("Downloading..."));

	ProgressWidget = SNew(SNotificationProgressWidget)
		.ProgressText(FText::FromString("Downloading " + AssetName));

	// Set up the notification properties
	Info.bFireAndForget                   = false; // We want to control when it disappears
	Info.FadeOutDuration                  = 1.0f;  // Duration of the fade-out
	Info.ExpireDuration                   = 0.0f;  // How long it stays on the screen
	Info.bUseThrobber                     = true;  // Adds a spinning throbber to the notification
	Info.bUseSuccessFailIcons             = true;  // Adds success/failure icons
	Info.bAllowThrottleWhenFrameRateIsLow = false; // Ensures it updates even if the frame rate is low
	Info.bUseLargeFont                    = false; // Uses the default font size
	Info.ContentWidget                    = ProgressWidget;

	DownloadProgressNotification = FSlateNotificationManager::Get().AddNotification(Info);

	if (DownloadProgressNotification.IsValid() && ProgressWidget)
	{
		DownloadProgressNotification->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void IFabWorkflow::SetDownloadNotificationProgress(const float Progress) const
{
	if (Progress > 100.0f || Progress < 0.0f)
	{
		return;
	}
	if (DownloadProgressNotification.IsValid() && ProgressWidget)
	{
		ProgressWidget->SetProgressPercent(Progress);
	}
}

void IFabWorkflow::ExpireDownloadNotification(bool bSuccess) const
{
	if (DownloadProgressNotification.IsValid())
	{
		DownloadProgressNotification->SetCompletionState(bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
		DownloadProgressNotification->ExpireAndFadeout();
	}
}

void IFabWorkflow::CreateImportNotification()
{
	// Create the notification info
	FNotificationInfo Info(FText::FromString("Importing..."));

	// Set up the notification properties
	Info.bFireAndForget                   = false; // We want to control when it disappears
	Info.FadeOutDuration                  = 1.0f;  // Duration of the fade-out
	Info.ExpireDuration                   = 0.0f;  // How long it stays on the screen
	Info.bUseThrobber                     = true;  // Adds a spinning throbber to the notification
	Info.bUseSuccessFailIcons             = true;  // Adds success/failure icons
	Info.bAllowThrottleWhenFrameRateIsLow = false; // Ensures it updates even if the frame rate is low
	Info.bUseLargeFont                    = false; // Uses the default font size

	ImportProgressNotification = FSlateNotificationManager::Get().AddNotification(Info);

	if (ImportProgressNotification.IsValid())
	{
		ImportProgressNotification->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void IFabWorkflow::ExpireImportNotification(bool bSuccess) const
{
	if (ImportProgressNotification.IsValid())
	{
		ImportProgressNotification->SetCompletionState(bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
		ImportProgressNotification->ExpireAndFadeout();
	}
}