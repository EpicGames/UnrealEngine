// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericDragDropWorkflow.h"

#include "FabBrowser.h"
#include "FabBrowserApi.h"
#include "FabLog.h"

#include "AssetRegistry/AssetRegistryModule.h"

#include "Components/MeshComponent.h"

#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"

#include "Materials/MaterialInstanceDynamic.h"

#include "Utilities/AssetUtils.h"
#include "Utilities/DragImportOperation.h"

FGenericDragDropWorkflow::FGenericDragDropWorkflow(const FString& InAssetId, const FString& InAssetName)
	: FGenericImportWorkflow(InAssetId, InAssetName, "")
{
	bIsDragDropWorkflow = true;
}

void FGenericDragDropWorkflow::Execute()
{
	FString AssetImportFolder = AssetName.IsEmpty() ? AssetId : AssetName;
	FAssetUtils::SanitizeFolderName(AssetImportFolder);
	ImportLocation = "/Game/Fab" / AssetImportFolder;

	if (FAssetData CachedMeshData; CheckForCachedAsset(ImportLocation, CachedMeshData))
	{
		DragOperation = MakeUnique<FDragImportOperation>(CachedMeshData, EDragAssetType::Mesh);
		FGenericImportWorkflow::CompleteWorkflow();
		return;
	}

	FAssetData PlaceHolderAsset(FSoftObjectPath("/Fab/Placeholders/MeshPlaceholder.MeshPlaceholder").TryLoad());
	DragOperation = MakeUnique<FDragImportOperation>(PlaceHolderAsset, EDragAssetType::Mesh);

	TWeakObjectPtr<UFabBrowserApi> WeakApi = FFabBrowser::GetBrowserApi();
	UFabBrowserApi* FabBrowserApi = WeakApi.Get();
	if (!FabBrowserApi)
	{
		return;
	}

	auto OnDragInfoSuccess = [this, WeakApi](const FString& InDownloadUrl, const FFabAssetMetadata& AssetMetadata)
	{
		if (AssetId == AssetMetadata.AssetId)
		{
			if (InDownloadUrl.IsEmpty())
			{
				CancelWorkflow();
			}
			else
			{
				DownloadUrl = InDownloadUrl;
				DownloadContent();
			}
			if (SignedUrlHandle.IsValid())
			{
				if (UFabBrowserApi* Api = WeakApi.Get())
				{
					Api->RemoveSignedUrlHandle(SignedUrlHandle);
				}
				SignedUrlHandle.Reset();
			}
		}
	};

	SignedUrlHandle = FabBrowserApi->AddSignedUrlCallback(OnDragInfoSuccess);
}

void FGenericDragDropWorkflow::OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	FGenericImportWorkflow::OnContentDownloadProgress(Request, DownloadStats);

	if (DownloadStats.PercentComplete > 100.0f || DownloadStats.PercentComplete < 0.0f)
	{
		return;
	}
	if (DragOperation)
	{
		const TWeakObjectPtr<AActor>& SpawnedActor = DragOperation->GetSpawnedActor();
		if (!SpawnedActor.IsValid()) return;

		if (UMeshComponent* const MeshComponent = SpawnedActor->GetComponentByClass<UMeshComponent>())
		{
			if (MeshComponent->GetNumOverrideMaterials() == 0)
			{
				MeshComponent->SetMaterial(0, UMaterialInstanceDynamic::Create(MeshComponent->GetMaterial(0), MeshComponent));
			}
			if (UMaterialInstanceDynamic* const Material = Cast<UMaterialInstanceDynamic>(MeshComponent->GetMaterial(0)))
			{
				Material->SetScalarParameterValue("Progress", DownloadStats.PercentComplete / 100.0f);
			}
		}
	}
}

void FGenericDragDropWorkflow::CompleteWorkflow()
{
	if (UStaticMesh* ImportedMesh = GetImportedObjectOfType<UStaticMesh>())
	{
		DragOperation->UpdateDraggedAsset(ImportedMesh, EDragAssetType::Mesh);
		FGenericImportWorkflow::CompleteWorkflow();
	}
	else if (USkeletalMesh* ImportedSkeletalMesh = GetImportedObjectOfType<USkeletalMesh>())
	{
		DragOperation->UpdateDraggedAsset(ImportedSkeletalMesh, EDragAssetType::Mesh);
		FGenericImportWorkflow::CompleteWorkflow();
	}
	else
	{
		FAB_LOG_ERROR("Drag and Drop failed for FAB Asset %s", *AssetName);
		CancelWorkflow();
	}
}

void FGenericDragDropWorkflow::CancelWorkflow()
{
	if (DragOperation)
	{
		DragOperation->DeleteSpawnedActor();
		DragOperation->CancelOperation();
	}
	FGenericImportWorkflow::CancelWorkflow();
}

bool FGenericDragDropWorkflow::CheckForCachedAsset(const FString& SearchPath, FAssetData& CachedMeshData) const
{
	FARFilter Filter;
	Filter.PackagePaths.Add(FName(*SearchPath));
	Filter.ClassPaths.Add(USkeletalMesh::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(UStaticMesh::StaticClass()->GetClassPathName());
	Filter.bRecursivePaths = true;

	TArray<FAssetData> AssetDataList;
	IAssetRegistry::Get()->GetAssets(Filter, AssetDataList);
	if (!AssetDataList.IsEmpty())
	{
		CachedMeshData = MoveTemp(AssetDataList[0]);
		return true;
	}
	return false;
}
