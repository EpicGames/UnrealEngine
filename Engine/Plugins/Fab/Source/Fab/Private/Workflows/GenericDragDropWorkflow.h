// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericImportWorkflow.h"

#include "Utilities/DragImportOperation.h"

class FGenericDragDropWorkflow : public FGenericImportWorkflow
{
public:
	FGenericDragDropWorkflow(const FString& InAssetId, const FString& InAssetName);

	virtual void Execute() override;

	virtual void OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;

	virtual void CompleteWorkflow() override;
	virtual void CancelWorkflow() override;

private:
	bool CheckForCachedAsset(const FString& SearchPath, FAssetData& CachedMeshData) const;

private:
	FDelegateHandle SignedUrlHandle;

	TUniquePtr<FDragImportOperation> DragOperation;
};
