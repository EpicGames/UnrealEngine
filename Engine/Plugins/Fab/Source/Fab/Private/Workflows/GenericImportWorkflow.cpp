// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericImportWorkflow.h"

#include "FabLog.h"

#include "Engine/StaticMesh.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

#include "Importers/GenericAssetImporter.h"

#include "Misc/MessageDialog.h"

#include "Utilities/AssetUtils.h"
#include "Utilities/FabAssetsCache.h"
#include "Utilities/FabLocalAssets.h"

FGenericImportWorkflow::FGenericImportWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InDownloadURL)
	: IFabWorkflow(InAssetId, InAssetName, InDownloadURL)
{}

void FGenericImportWorkflow::Execute()
{
	DownloadContent();
}

void FGenericImportWorkflow::DownloadContent()
{
	CreateDownloadNotification();

	const FString DownloadLocation = FFabAssetsCache::GetCacheLocation() / AssetId;

	DownloadRequest = MakeShared<FFabDownloadRequest>(AssetId, DownloadUrl, DownloadLocation, EFabDownloadType::HTTP);
	DownloadRequest->OnDownloadProgress().AddSP(this, &FGenericImportWorkflow::OnContentDownloadProgress);
	DownloadRequest->OnDownloadComplete().AddSP(this, &FGenericImportWorkflow::OnContentDownloadComplete);
	DownloadRequest->ExecuteRequest();
}

void FGenericImportWorkflow::OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	SetDownloadNotificationProgress(DownloadStats.PercentComplete);
}

void FGenericImportWorkflow::OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	if (!DownloadStats.bIsSuccess)
	{
		FAB_LOG_ERROR("Failed to download FAB Asset %s", *AssetName);
		ExpireDownloadNotification(false);
		CancelWorkflow();
		return;
	}
	ExpireDownloadNotification(true);

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();

	const FString& DownloadedFile = DownloadStats.DownloadedFiles[0];
	if (DownloadedFile.EndsWith(".zip") || DownloadedFile.EndsWith(".rar"))
	{
		auto UnzipFileHelper = [this](const FString& ZipFilename, const FString ExtractLocation)
		{
			if (!FAssetUtils::Unzip(ZipFilename, ExtractLocation))
			{
				FAB_LOG_ERROR("Failed to unzip FAB Asset %s", *AssetName);
				ExpireDownloadNotification(false);
				return false;
			}
			return true;
		};

		const FString ExtractLocation = FPaths::GetBaseFilename(DownloadedFile, false) + "_extracted";
		if (!UnzipFileHelper(DownloadedFile, ExtractLocation))
		{
			CancelWorkflow();
			return;
		}

		// Check if there are still any zips inside the extracted folder
		TArray<FString> ZipFiles;
		FileManager.FindFilesRecursively(ZipFiles, *ExtractLocation, TEXT(".rar"));
		if (!ZipFiles.IsEmpty())
		{
			FAB_LOG_ERROR("'.rar' extract support is unavailable. Asset: %s", *AssetName);
			if (FMessageDialog::Open(
				EAppMsgCategory::Warning,
				EAppMsgType::YesNo,
				FText::FromString(
					"Some files will not be imported as '.rar' extract support is unavailable.\nDo you want to open the file to manually Extract and Import?")
			) == EAppReturnType::Yes)
			{
				FPlatformProcess::ExploreFolder(*ZipFiles[0]);
			}
			ZipFiles.Empty();
		}
		FileManager.FindFilesRecursively(ZipFiles, *ExtractLocation, TEXT(".zip"));
		for (const FString& ZipFile : ZipFiles)
		{
			if (!UnzipFileHelper(ZipFile, FPaths::GetBaseFilename(ZipFile, false) + "_extracted"))
			{
				CancelWorkflow();
				return;
			}
		}
	}

	TArray<FString> ImportFiles;
	const FString SearchPath = FPaths::GetPath(DownloadedFile);
	FileManager.FindFilesRecursively(ImportFiles, *SearchPath, TEXT("gltf"));
	FileManager.FindFilesRecursively(ImportFiles, *SearchPath, TEXT("glb"));
	if (ImportFiles.IsEmpty())
	{
		const TSet<FString> MeshImportExtensions = {
			"fbx",
			"obj",
			"usdz",
		};
		const TSet<FString> TextureImportExtensions = {
			"jpg",
			"jpeg",
			"png",
			"exr",
			"bmp",
			"tif",
			"tiff",
			"webp",
		};
		bool bEmbeddedTextures = false;
		FileManager.IterateDirectoryRecursively(
			*SearchPath,
			[&](const TCHAR* FilenameOrDirectory, const bool bIsDirectory)
			{
				if (bIsDirectory)
				{
					if (FCString::Strfind(FilenameOrDirectory, TEXT(".fbm")))
						bEmbeddedTextures = true;
					return true;
				}
				const FString Extension = FPaths::GetExtension(FilenameOrDirectory);
				if (MeshImportExtensions.Contains(Extension) || TextureImportExtensions.Contains(Extension))
				{
					const bool bExist = ImportFiles.ContainsByPredicate(
						[CleanFilename = FPaths::GetCleanFilename(FilenameOrDirectory)](const FString& Path)
						{
							if (FPaths::GetCleanFilename(Path) == CleanFilename)
								return true;
							return false;
						}
					);
					if (!bExist)
						ImportFiles.Add(FilenameOrDirectory);
				}
				return true;
			}
		);
		if (bEmbeddedTextures)
		{
			ImportFiles.RemoveAll(
				[&](const FString& Path)
				{
					if (TextureImportExtensions.Contains(FPaths::GetExtension(Path)))
						return true;
					return false;
				}
			);
		}
	}

	const auto IsSameFilename = [](const FString& A, const FString& B) { return FPaths::GetBaseFilename(A) == FPaths::GetBaseFilename(B); };
	TArray<FString> UpdatedImportFiles;
	UpdatedImportFiles.Reserve(ImportFiles.Num());
	for (FString& ImportFile : ImportFiles)
	{
		if (UpdatedImportFiles.ContainsByPredicate([&](const FString& Path) { return IsSameFilename(ImportFile, Path); }))
		{
			const FString Extension = FPaths::GetExtension(ImportFile);
			const FString NewFile = FPaths::GetBaseFilename(ImportFile, false) + '_' + Extension + '.' + Extension;
			if (FileManager.MoveFile(*NewFile, *ImportFile))
				ImportFile = NewFile;
		}

		if (FPaths::FileExists(ImportFile))
			UpdatedImportFiles.Add(ImportFile);
	}

	if (UpdatedImportFiles.IsEmpty())
	{
		FAB_LOG_ERROR("Import files not found for %s", *AssetName);
		ExpireDownloadNotification(false);
		CancelWorkflow();
		return;
	}

	ImportContent(UpdatedImportFiles);
}

void FGenericImportWorkflow::CompleteWorkflow()
{
	FAssetUtils::SyncContentBrowserToFolder(ImportLocation, !bIsDragDropWorkflow);
	IFabWorkflow::CompleteWorkflow();
}

void FGenericImportWorkflow::ImportContent(const TArray<FString>& ImportFiles)
{
	FString AssetImportFolder = AssetName.IsEmpty() ? AssetId : AssetName;
	FAssetUtils::SanitizeFolderName(AssetImportFolder);

	ImportLocation = "/Game/Fab" / AssetImportFolder;

	FFabGenericImporter::ImportAsset(
		ImportFiles,
		ImportLocation,
		[this](const TArray<UObject*>& Objects)
		{
			if (!Objects.IsEmpty())
			{
				ImportedObjects = Objects;
				UFabLocalAssets::AddLocalAsset(ImportLocation, AssetId);
				ExpireImportNotification(true);
				CompleteWorkflow();
			}
			else
			{
				FAB_LOG_ERROR("Asset import failed: %s", *AssetName);
				ExpireImportNotification(false);
				CancelWorkflow();
			}
		}
	);
}
