// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackImportWorkflow.h"

#include "FabDownloader.h"
#include "FabLog.h"
#include "NotificationProgressWidget.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Framework/Docking/TabManager.h" 
#include "Framework/Notifications/NotificationManager.h"

#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"

#include "UnrealEdMisc.h"

#include "Utilities/AssetUtils.h"
#include "Utilities/FabLocalAssets.h"

#include "Widgets/Notifications/SNotificationList.h"

FPackImportWorkflow::FPackImportWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InManifestDownloadUrl, const FString& InBaseUrls)
	: IFabWorkflow(InAssetId, InAssetName, InManifestDownloadUrl)
	, BaseUrls(InBaseUrls)
{}

void FPackImportWorkflow::Execute()
{
	DownloadContent();
}

void FPackImportWorkflow::DownloadContent()
{
	const FString DownloadURL      = DownloadUrl + ',' + BaseUrls;
	const FString DownloadLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir());

	DownloadRequest = MakeShared<FFabDownloadRequest>(AssetId, DownloadURL, DownloadLocation, EFabDownloadType::BuildPatchRequest);
	DownloadRequest->OnDownloadComplete().AddSP(this, &FPackImportWorkflow::OnContentDownloadComplete);
	DownloadRequest->OnDownloadProgress().AddSP(this, &FPackImportWorkflow::OnContentDownloadProgress);
	DownloadRequest->ExecuteRequest();

	CreateDownloadNotification();
}

void FPackImportWorkflow::OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	SetDownloadNotificationProgress(DownloadStats.PercentComplete);
}

TSharedPtr<IPlugin> FindPluginOwningModule(const FString& ModuleName)
{
	for (const TSharedRef<IPlugin>& Plugin : IPluginManager::Get().GetDiscoveredPlugins())
	{
		const FPluginDescriptor& Desc = Plugin->GetDescriptor();
		for (const FModuleDescriptor& Mod : Desc.Modules)
		{
			if (Mod.Name.ToString().Equals(ModuleName, ESearchCase::IgnoreCase))
			{
				return Plugin;
			}
		}
	}

	return nullptr;
}

void PromptToEnablePlugins(const TArray<FString>& PluginNames, const TMap<FString, TArray<FString>>& PluginToAffectedAssets)
{
	if (PluginNames.IsEmpty())
	{
		return;
	}

	FString PluginList;
	for (const FString& PluginName : PluginNames)
	{
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName);
		PluginList += (Plugin.IsValid() ? Plugin->GetFriendlyName() : PluginName) + TEXT("\n");
	}

	FString AssetList;
	for (const FString& PluginName : PluginNames)
	{
		if (const TArray<FString>* Assets = PluginToAffectedAssets.Find(PluginName))
		{
			for (const FString& AssetPath : *Assets)
			{
				AssetList += AssetPath + TEXT("\n");
			}
		}
	}

	const FText TitleText = NSLOCTEXT("Fab", "MissingPluginsTitle", "Missing Plugins!");
	const FText SubText = FText::Format(
		NSLOCTEXT("Fab", "MissingPluginsSubText", "Needed Plugins:\n{0}\nAffected Assets:\n{1}\nThe imported assets need the plugins listed above. Related assets may not display properly.\nAttempting to save these assets may result in irreversible modification due to missing plugins."),
		FText::AsCultureInvariant(PluginList),
		FText::AsCultureInvariant(AssetList));

	TSharedPtr<TWeakPtr<SNotificationItem>> NotificationWeak = MakeShared<TWeakPtr<SNotificationItem>>();

	FNotificationInfo Info(TitleText);
	Info.bFireAndForget = false;
	Info.FadeOutDuration = 0.0f;
	Info.ExpireDuration = 0.0f;
	Info.WidthOverride = FOptionalSize();
	Info.SubText = SubText;

	Info.HyperlinkText = NSLOCTEXT("Fab", "OpenPluginBrowser", "Open Plugin Browser");
	Info.Hyperlink = FSimpleDelegate::CreateLambda([]()
	{
		FGlobalTabmanager::Get()->TryInvokeTab(FName("PluginsEditor"));
	});

	Info.ButtonDetails.Add(FNotificationButtonInfo(
		NSLOCTEXT("Fab", "EnableMissingAndRestart", "Enable Missing and Restart"),
		NSLOCTEXT("Fab", "EnableMissingAndRestartTT", "Enable missing plugins and restart the editor"),
		FSimpleDelegate::CreateLambda([PluginNames, NotificationWeak]()
		{
			TArray<FString> EnabledPlugins;
			bool bEnableSuccess = true;
			for (const FString& PluginName : PluginNames)
			{
				FText FailReason;
				if (!IProjectManager::Get().SetPluginEnabled(PluginName, true, FailReason))
				{
					UE_LOGF(LogFab, Error, "Failed to enable plugin '%ls': %ls", *PluginName, *FailReason.ToString());
					bEnableSuccess = false;
					break;
				}
				EnabledPlugins.Add(PluginName);
			}

			if (!bEnableSuccess)
			{
				// Rollback successfully enabled plugins to restore consistent state
				for (int32 i = EnabledPlugins.Num() - 1; i >= 0; --i)
				{
					FText RollbackReason;
					if (!IProjectManager::Get().SetPluginEnabled(EnabledPlugins[i], false, RollbackReason))
					{
						UE_LOGF(LogFab, Error, "Failed to rollback plugin '%ls': %ls", *EnabledPlugins[i], *RollbackReason.ToString());
					}
				}

				if (const TSharedPtr<SNotificationItem> Notification = NotificationWeak->Pin())
				{
					Notification->SetCompletionState(SNotificationItem::CS_Fail);
					Notification->ExpireAndFadeout();
				}
				return;
			}

			if (IProjectManager::Get().IsCurrentProjectDirty())
			{
				FText FailReason;
				if (!IProjectManager::Get().SaveCurrentProjectToDisk(FailReason))
				{
					UE_LOGF(LogFab, Error, "Failed to save project: %ls. Restarting to reconcile state.", *FailReason.ToString());
				}
			}

			if (const TSharedPtr<SNotificationItem> Notification = NotificationWeak->Pin())
			{
				Notification->SetCompletionState(SNotificationItem::CS_Success);
				Notification->ExpireAndFadeout();
			}

			FUnrealEdMisc::Get().RestartEditor(false);
		}),
		SNotificationItem::CS_None
	));

	Info.ButtonDetails.Add(FNotificationButtonInfo(
		NSLOCTEXT("Fab", "DismissPluginPrompt", "Dismiss"),
		NSLOCTEXT("Fab", "DismissPluginPromptTT", "Dismiss this notification"),
		FSimpleDelegate::CreateLambda([NotificationWeak]()
		{
			if (const TSharedPtr<SNotificationItem> Notification = NotificationWeak->Pin())
			{
				Notification->SetCompletionState(SNotificationItem::CS_None);
				Notification->ExpireAndFadeout();
			}
		}),
		SNotificationItem::CS_None
	));

	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
	*NotificationWeak = Notification;

	if (Notification.IsValid())
	{
		Notification->SetCompletionState(SNotificationItem::CS_None);
	}
}

void CheckForDependencies(const TArray<FString>& ImportedFiles)
{
	if (ImportedFiles.Num() == 0)
	{
		UE_LOGF(LogFab, Warning, "No files imported");
		return;
	}

	// Gather all .uasset files
	TArray<FString> FoundFiles;
	for (const FString& File : ImportedFiles)
	{
		if (File.EndsWith(TEXT(".uasset")))
		{
			const FString FullPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectDir() / File);
			FoundFiles.AddUnique(FullPath);
			UE_LOGF(LogFab, Log, "Found %ls", *FullPath);
		}
	}

	// Load Asset Registry
	const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	const IAssetRegistry& Registry = AssetRegistryModule.Get();

	UE_LOGF(LogFab, Display, "Scanning %d assets for plugin dependencies...\n", FoundFiles.Num());

	TArray<FString> PluginsToEnable;
	TMap<FString, TArray<FString>> PluginToAffectedAssets;

	for (const FString& FilePath : FoundFiles)
	{
		FString LongPackageName;
		if (!FPackageName::TryConvertFilenameToLongPackageName(FilePath, LongPackageName))
		{
			UE_LOGF(LogFab, Warning, "Failed to convert to package name: %ls", *FilePath);
			continue;
		}

		const FName PackageName(*LongPackageName);

		TArray<FAssetData> Assets;
		Registry.GetAssetsByPackageName(PackageName, Assets);

		for (const FAssetData& AssetData : Assets)
		{
			const FString ClassPath = AssetData.AssetClassPath.ToString();
			if (!ClassPath.StartsWith(TEXT("/Script/")))
			{
				continue;
			}

			// Extract module name from /Script/Module.Class
			FString Remainder = ClassPath.RightChop(8); // remove "/Script/"

			FString ModuleName;
			Remainder.Split(TEXT("."), &ModuleName, nullptr);

			if (FModuleManager::Get().IsModuleLoaded(*ModuleName))
			{
				continue;
			}

			const TSharedPtr<IPlugin> Plugin = FindPluginOwningModule(ModuleName);

			if (!Plugin.IsValid())
			{
				continue;
			}

			if (Plugin->IsEnabled())
			{
				// Already enabled — nothing to do
				continue;
			}

			const FString PluginName = Plugin->GetName();
			PluginsToEnable.AddUnique(PluginName);
			PluginToAffectedAssets.FindOrAdd(PluginName).AddUnique(LongPackageName);
		}
	}

	if (PluginsToEnable.Num() > 0)
	{
		PromptToEnablePlugins(PluginsToEnable, PluginToAffectedAssets);
	}
}

void FPackImportWorkflow::OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	if (!DownloadStats.bIsSuccess || DownloadStats.DownloadedFiles.IsEmpty())
	{
		ExpireDownloadNotification(false);
		CancelWorkflow();
		return;
	}

	ExpireDownloadNotification(true);

	TArray<FString> PathParts;
	DownloadStats.DownloadedFiles[0].ParseIntoArray(PathParts, TEXT("/"));

	if (PathParts.Num() >= 2)
	{
		ImportLocation = "/Game" / PathParts[1];
		UFabLocalAssets::AddLocalAsset(ImportLocation, AssetId);
		FAssetUtils::ScanForAssets(ImportLocation);
		FAssetUtils::SyncContentBrowserToFolder(ImportLocation);
	}

	// Check for dependencies
	CheckForDependencies(DownloadStats.DownloadedFiles);

	CompleteWorkflow();
}

void FPackImportWorkflow::CreateDownloadNotification()
{
	// Create the notification info
	FNotificationInfo Info(FText::FromString("Downloading..."));

	TWeakPtr<FFabDownloadRequest> WeakRequest = DownloadRequest;
	ProgressWidget = SNew(SNotificationProgressWidget)
		.ProgressText(FText::FromString("Downloading " + AssetName))
		.HasButton(true)
		.ButtonText(FText::FromString("Cancel"))
		.ButtonToolTip(FText::FromString("Cancel Pack Import"))
		.OnButtonClicked(
			FOnClicked::CreateLambda(
				[WeakRequest]()
				{
					FAB_LOG("Import Cancelled");
					if (TSharedPtr<FFabDownloadRequest> PinnedRequest = WeakRequest.Pin())
					{
						PinnedRequest->Cancel();
					}
					return FReply::Handled();
				}
			)
		);

	// Set up the notification properties
	Info.bFireAndForget                   = false; // We want to control when it disappears
	Info.FadeOutDuration                  = 1.0f;  // Duration of the fade-out
	Info.ExpireDuration                   = 0.0f;  // How long it stays on the screen
	Info.bUseThrobber                     = true;  // Adds a spinning throbber to the notification
	Info.bUseSuccessFailIcons             = true;  // Adds success/failure icons
	Info.bAllowThrottleWhenFrameRateIsLow = false; // Ensures it updates even if the frame rate is low
	Info.bUseLargeFont                    = false; // Uses the default font size
	Info.ContentWidget                    = ProgressWidget;

	DownloadProgressNotification = FSlateNotificationManager::Get().AddNotification(Info);

	if (DownloadProgressNotification.IsValid() && ProgressWidget)
	{
		DownloadProgressNotification->SetCompletionState(SNotificationItem::CS_Pending);
	}
}
