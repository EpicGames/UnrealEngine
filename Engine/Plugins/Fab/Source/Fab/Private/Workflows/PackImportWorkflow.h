// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"

#include "Workflows/FabWorkflow.h"

class SNotificationItem;
class SNotificationProgressWidget;

class FPackImportWorkflow : public IFabWorkflow
{
public:
	FPackImportWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InManifestDownloadUrl, const FString& InBaseUrls);

	virtual void Execute() override;

protected:
	virtual void DownloadContent() override;

	virtual void OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;
	virtual void OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;

	virtual void CreateDownloadNotification() override;

private:
	FString BaseUrls;

	TSharedPtr<FFabDownloadRequest> DownloadRequest;
};
