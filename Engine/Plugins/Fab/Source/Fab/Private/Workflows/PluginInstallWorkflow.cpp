// Copyright Epic Games, Inc. All Rights Reserved.

#include "PluginInstallWorkflow.h"

#include "FabDownloader.h"
#include "FabLog.h"
#include "NotificationProgressWidget.h"

#include "Framework/Notifications/NotificationManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "Interfaces/IPluginManager.h"
#include "Interfaces/IProjectManager.h"
#include "Misc/Paths.h"
#include "UnrealEdMisc.h"

#include "Widgets/Notifications/SNotificationList.h"

FPluginInstallWorkflow::FPluginInstallWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InManifestDownloadUrl, const FString& InBaseUrls, bool bInInstallToEngine)
	: IFabWorkflow(InAssetId, InAssetName, InManifestDownloadUrl)
	, BaseUrls(InBaseUrls)
	, bInstallToEngine(bInInstallToEngine)
{}

void FPluginInstallWorkflow::Execute()
{
	DownloadContent();
}

void FPluginInstallWorkflow::DownloadContent()
{
	const FString DownloadURL = DownloadUrl + ',' + BaseUrls;

	if (bInstallToEngine)
	{
		// Download to the engine root so manifest paths (Engine/Plugins/Marketplace/X/...)
		// resolve to the correct engine location without any relocation.
		DownloadLocation = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() / TEXT(".."));
	}
	else
	{
		DownloadLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectPluginsDir());
	}

	DownloadRequest = MakeShared<FFabDownloadRequest>(AssetId, DownloadURL, DownloadLocation, EFabDownloadType::BuildPatchInstallRequest);
	DownloadRequest->OnDownloadComplete().AddSP(this, &FPluginInstallWorkflow::OnContentDownloadComplete);
	DownloadRequest->OnDownloadProgress().AddSP(this, &FPluginInstallWorkflow::OnContentDownloadProgress);
	DownloadRequest->ExecuteRequest();

	CreateDownloadNotification();
}

void FPluginInstallWorkflow::OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	SetDownloadNotificationProgress(DownloadStats.PercentComplete);
}

void FPluginInstallWorkflow::OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	if (!DownloadStats.bIsSuccess || DownloadStats.DownloadedFiles.IsEmpty())
	{
		ExpireDownloadNotification(false);
		CancelWorkflow();
		return;
	}

	FString InstalledPluginName;
	if (bInstallToEngine)
	{
		// Files already landed in Engine/Plugins/Marketplace/X/ — no relocation needed.
		const FString* UpluginRelPath = DownloadStats.DownloadedFiles.FindByPredicate(
			[](const FString& File) {
				const FString Ext = FPaths::GetExtension(File);
				// Also check for "uplugi" due to BPS off-by-one bug truncating filenames
				return Ext == TEXT("uplugin") || Ext == TEXT("uplugi");
			}
		);
		if (!UpluginRelPath)
		{
			FAB_LOG_ERROR("Could not find .uplugin file in downloaded manifest");
			ExpireDownloadNotification(false);
			CancelWorkflow();
			return;
		}
		InstalledPluginName = FPaths::GetBaseFilename(*UpluginRelPath);
	}
	else
	{
		if (!RelocatePlugin(DownloadStats, InstalledPluginName))
		{
			FAB_LOG_ERROR("Failed to relocate plugin to project plugins directory");
			ExpireDownloadNotification(false);
			CancelWorkflow();
			return;
		}
	}

	AsyncTask(ENamedThreads::GameThread, [InstalledPluginName, bInstallToEngine = bInstallToEngine]()
	{
		const TSharedPtr<IPlugin> ExistingPlugin = IPluginManager::Get().FindPlugin(InstalledPluginName);
		if (ExistingPlugin.IsValid() && ExistingPlugin->IsMounted())
		{
			FAB_LOG("Plugin '%s' is already mounted, skipping", *InstalledPluginName);
			return;
		}

		IPluginManager::Get().MountNewlyCreatedPlugin(InstalledPluginName);

		TSharedPtr<TWeakPtr<SNotificationItem>> NotificationWeak = MakeShared<TWeakPtr<SNotificationItem>>();

		FNotificationInfo Info(FText::Format(
			NSLOCTEXT("Fab", "PluginInstalled", "{0} installed. Restart the editor to use it."),
			FText::FromString(InstalledPluginName)));
		Info.bFireAndForget = false;

		Info.ButtonDetails.Add(FNotificationButtonInfo(
			bInstallToEngine
				? NSLOCTEXT("Fab", "EnableAndRestart", "Enable and Restart")
				: NSLOCTEXT("Fab", "RestartNow", "Restart Now"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateLambda([InstalledPluginName, bInstallToEngine]()
			{
				// Engine-installed plugins need explicit .uproject enablement;
				// project-installed plugins are auto-discovered on restart.
				if (bInstallToEngine)
				{
					FText FailReason;
					if (!IProjectManager::Get().SetPluginEnabled(InstalledPluginName, true, FailReason))
					{
						FAB_LOG_ERROR("Failed to enable plugin '%s': %s", *InstalledPluginName, *FailReason.ToString());
						return;
					}

					if (IProjectManager::Get().IsCurrentProjectDirty())
					{
						if (!IProjectManager::Get().SaveCurrentProjectToDisk(FailReason))
						{
							FAB_LOG_ERROR("Failed to save project after enabling plugin '%s': %s. Restarting to reconcile state.", *InstalledPluginName, *FailReason.ToString());
						}
					}
				}
				FUnrealEdMisc::Get().RestartEditor(false);
			}),
			SNotificationItem::CS_None
		));

		Info.ButtonDetails.Add(FNotificationButtonInfo(
			NSLOCTEXT("Fab", "RestartLater", "Restart Later"),
			FText::GetEmpty(),
			FSimpleDelegate::CreateLambda([NotificationWeak]()
			{
				if (const TSharedPtr<SNotificationItem> Notification = NotificationWeak->Pin())
				{
					Notification->ExpireAndFadeout();
				}
			}),
			SNotificationItem::CS_None
		));

		TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);
		*NotificationWeak = Notification;
	});

	ExpireDownloadNotification(true);

	CompleteWorkflow();
}

bool FPluginInstallWorkflow::RelocatePlugin(const FFabDownloadStats& DownloadStats, FString& OutPluginName) const
{
	const FString* UPluginFilePath = DownloadStats.DownloadedFiles.FindByPredicate(
		[](const FString& File)
		{
			const FString Ext = FPaths::GetExtension(File);
			// Also check for "uplugi" due to BPS off-by-one bug truncating filenames
			return Ext == TEXT("uplugin") || Ext == TEXT("uplugi");
		}
	);

	if (!UPluginFilePath)
	{
		FAB_LOG_ERROR("Could not find .uplugin file in downloaded manifest");
		return false;
	}

	// "Engine/Plugins/Marketplace/X/X.uplugin" -> dir "Engine/Plugins/Marketplace/X", name "X"
	const FString PluginDir  = FPaths::GetPath(*UPluginFilePath);
	const FString PluginName    = FPaths::GetBaseFilename(*UPluginFilePath);
	const FString PluginSrcDir  = DownloadLocation / PluginDir;
	const FString PluginDestDir = DownloadLocation / TEXT("Marketplace") / PluginName;

	OutPluginName = PluginName;

	if (PluginSrcDir == PluginDestDir)
	{
		return true; // Already in the right place
	}

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	if (!PlatformFile.DirectoryExists(*PluginSrcDir))
	{
		FAB_LOG_ERROR("Plugin source directory not found after install: %s", *PluginSrcDir);
		return false;
	}

	FAB_LOG("Relocating plugin from '%s' to '%s'", *PluginSrcDir, *PluginDestDir);

	if (!PlatformFile.CopyDirectoryTree(*PluginDestDir, *PluginSrcDir, true))
	{
		FAB_LOG_ERROR("Failed to copy plugin directory to '%s'", *PluginDestDir);
		return false;
	}

	// Remove the source directory now that files are copied
	IFileManager::Get().DeleteDirectory(*PluginSrcDir, false, true);

	// Clean up the leftover nested parent directories (e.g. "Engine/" under DownloadLocation)
	int32 SlashIdx;
	if (PluginDir.FindChar(TEXT('/'), SlashIdx))
	{
		const FString NestedRoot = DownloadLocation / PluginDir.Left(SlashIdx);
		IFileManager::Get().DeleteDirectory(*NestedRoot, false, true);
	}

	return true;
}

void FPluginInstallWorkflow::CreateDownloadNotification()
{
	// Create the notification info
	FNotificationInfo Info(FText::FromString("Downloading..."));

	TWeakPtr<FFabDownloadRequest> WeakRequest = DownloadRequest;
	ProgressWidget = SNew(SNotificationProgressWidget)
		.ProgressText(FText::FromString("Downloading " + AssetName))
		.HasButton(true)
		.ButtonText(FText::FromString("Cancel"))
		.ButtonToolTip(FText::FromString("Cancel Plugin Install"))
		.OnButtonClicked(
			FOnClicked::CreateLambda(
				[WeakRequest]()
				{
					FAB_LOG("Install Cancelled");
					if (TSharedPtr<FFabDownloadRequest> PinnedRequest = WeakRequest.Pin())
					{
						PinnedRequest->Cancel();
					}
					return FReply::Handled();
				}
			)
		);

	// Set up the notification properties
	Info.bFireAndForget                   = false; // We want to control when it disappears
	Info.FadeOutDuration                  = 1.0f;  // Duration of the fade-out
	Info.ExpireDuration                   = 0.0f;  // How long it stays on the screen
	Info.bUseThrobber                     = true;  // Adds a spinning throbber to the notification
	Info.bUseSuccessFailIcons             = true;  // Adds success/failure icons
	Info.bAllowThrottleWhenFrameRateIsLow = false; // Ensures it updates even if the frame rate is low
	Info.bUseLargeFont                    = false; // Uses the default font size
	Info.ContentWidget                    = ProgressWidget;

	DownloadProgressNotification = FSlateNotificationManager::Get().AddNotification(Info);

	if (DownloadProgressNotification.IsValid() && ProgressWidget)
	{
		DownloadProgressNotification->SetCompletionState(SNotificationItem::CS_Pending);
	}
}

void FPluginInstallWorkflow::SetDownloadNotificationProgress(const float Progress) const
{
	if (Progress > 100.0f || Progress < 0.0f)
	{
		return;
	}
	if (DownloadProgressNotification.IsValid() && ProgressWidget)
	{
		ProgressWidget->SetProgressPercent(Progress);
	}
}

void FPluginInstallWorkflow::ExpireDownloadNotification(bool bSuccess) const
{
	if (DownloadProgressNotification.IsValid())
	{
		DownloadProgressNotification->SetCompletionState(bSuccess ? SNotificationItem::CS_Success : SNotificationItem::CS_Fail);
		DownloadProgressNotification->ExpireAndFadeout();
	}
}
