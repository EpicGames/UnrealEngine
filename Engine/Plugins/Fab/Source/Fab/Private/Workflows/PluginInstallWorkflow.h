// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Workflows/FabWorkflow.h"

class SNotificationItem;
class SNotificationProgressWidget;

class FPluginInstallWorkflow : public IFabWorkflow
{
public:
	FPluginInstallWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InManifestDownloadUrl, const FString& InBaseUrls, bool bInInstallToEngine);

	virtual void Execute() override;

protected:
	virtual void DownloadContent() override;

	virtual void OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;
	virtual void OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;

private:
	virtual void CreateDownloadNotification() override;
	virtual void SetDownloadNotificationProgress(const float Progress) const override;
	virtual void ExpireDownloadNotification(bool bSuccess) const override;

	bool RelocatePlugin(const FFabDownloadStats& DownloadStats, FString& OutPluginName) const;

	FString BaseUrls;
	FString DownloadLocation;
	bool bInstallToEngine = false;
	TSharedPtr<FFabDownloadRequest> DownloadRequest;
	TSharedPtr<SNotificationItem> DownloadProgressNotification;
	TSharedPtr<SNotificationItem> ImportProgressNotification;
	TSharedPtr<SNotificationProgressWidget> ProgressWidget;
};
