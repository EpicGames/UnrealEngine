// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuixelDragDropWorkflow.h"

#include "FabBrowser.h"
#include "FabBrowserApi.h"
#include "FabDownloader.h"
#include "FabLog.h"
#include "FabSettings.h"

#include "Components/MeshComponent.h"

#include "Engine/StaticMesh.h"

#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Utilities/DragImportOperation.h"
#include "Utilities/FabLocalAssets.h"

FQuixelDragDropWorkflow::FQuixelDragDropWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InListingType)
	: FQuixelImportWorkflow(InAssetId, InAssetName, "")
	, ListingType(InListingType)
{
	bIsDragDropWorkflow = true;
}

void FQuixelDragDropWorkflow::Execute()
{
	FAssetData PlaceHolderAsset;
	EDragAssetType DragAssetType;

	if (ListingType == "3d-model")
	{
		PlaceHolderAsset = FAssetData(FSoftObjectPath("/Fab/Placeholders/MeshPlaceholder.MeshPlaceholder").TryLoad());
		DragAssetType    = EDragAssetType::Mesh;
	}
	else if (ListingType == "material")
	{
		const FString SurfaceInstancePath = TEXT("/Fab/Materials/Standard/M_MS_Srf.M_MS_Srf");
		PlaceHolderAsset                  = IAssetRegistry::Get()->GetAssetByObjectPath(FSoftObjectPath(SurfaceInstancePath));
		DragAssetType                     = EDragAssetType::Material;
	}
	else if (ListingType == "decal")
	{
		const FString DecalInstancePath = TEXT("/Fab/Placeholders/DecalPlaceholder.DecalPlaceholder");
		PlaceHolderAsset                = IAssetRegistry::Get()->GetAssetByObjectPath(FSoftObjectPath(DecalInstancePath));
		DragAssetType                   = EDragAssetType::Decal;
	}
	else
	{
		FAB_LOG_ERROR("Listing type not supported: %s", *ListingType);
		return;
	}

	const FString* CachedPath = UFabLocalAssets::FindPath(AssetId);
	if (CachedPath && !CachedPath->IsEmpty())
	{
		const FString& ImportedAssetName = FPaths::GetPathLeaf(*CachedPath); // For example 3D
		const FString& Type              = FPaths::GetPathLeaf(CachedPath->Left(CachedPath->Len() - ImportedAssetName.Len() - 1));
		FAB_LOG("The type - %s", *Type);

		// Get the preferred quality tier from settings
		const UFabSettings* FabSettings = GetDefault<UFabSettings>();
		FString PreferredQuality        = "Medium";
		if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::Low)
		{
			PreferredQuality = "Low";
		}
		else if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::Medium)
		{
			PreferredQuality = "Medium";
		}
		else if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::High)
		{
			PreferredQuality = "High";
		}
		else if (FabSettings->PreferredQualityTier == EFabPreferredQualityTier::Raw)
		{
			PreferredQuality = "Raw";
		}

		const FString FullAssetPath = *CachedPath / PreferredQuality;
		FAB_LOG("Full path %s", *FullAssetPath);

		if (FAssetData CachedData; CheckForCachedAsset(FullAssetPath, CachedData))
		{
			DragOperation = MakeUnique<FDragImportOperation>(CachedData, DragAssetType);
			FQuixelImportWorkflow::CompleteWorkflow();
			return;
		}
	}

	DragOperation = MakeUnique<FDragImportOperation>(PlaceHolderAsset, DragAssetType);

	TWeakObjectPtr<UFabBrowserApi> WeakApi = FFabBrowser::GetBrowserApi();
	UFabBrowserApi* FabBrowserApi = WeakApi.Get();
	if (!FabBrowserApi)
	{
		return;
	}

	auto OnDragInfoSuccess = [this, WeakApi](const FString& InDownloadUrl, const FFabAssetMetadata& AssetMetadata)
	{
		if (AssetId == AssetMetadata.AssetId)
		{
			if (InDownloadUrl.IsEmpty())
			{
				CancelWorkflow();
			}
			else
			{
				DownloadUrl = InDownloadUrl;
				DownloadContent();
			}
			if (SignedUrlHandle.IsValid())
			{
				if (UFabBrowserApi* Api = WeakApi.Get())
				{
					Api->RemoveSignedUrlHandle(SignedUrlHandle);
				}
				SignedUrlHandle.Reset();
			}
		}
	};

	SignedUrlHandle = FabBrowserApi->AddSignedUrlCallback(OnDragInfoSuccess);
}

void FQuixelDragDropWorkflow::OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	FQuixelImportWorkflow::OnContentDownloadProgress(Request, DownloadStats);

	if (DownloadStats.PercentComplete > 100.0f || DownloadStats.PercentComplete < 0.0f)
	{
		return;
	}
	if (DragOperation)
	{
		const TWeakObjectPtr<AActor>& SpawnedActor = DragOperation->GetSpawnedActor();
		if (!SpawnedActor.IsValid()) return;

		if (UMeshComponent* const MeshComponent = SpawnedActor->GetComponentByClass<UMeshComponent>())
		{
			if (MeshComponent->GetNumOverrideMaterials() == 0)
			{
				MeshComponent->SetMaterial(0, UMaterialInstanceDynamic::Create(MeshComponent->GetMaterial(0), MeshComponent));
			}
			if (UMaterialInstanceDynamic* const Material = Cast<UMaterialInstanceDynamic>(MeshComponent->GetMaterial(0)))
			{
				Material->SetScalarParameterValue("Progress", DownloadStats.PercentComplete / 100.0f);
			}
		}
	}
}

void FQuixelDragDropWorkflow::CompleteWorkflow()
{
	const UObject* ImportedAsset = nullptr;
	EDragAssetType AssetType     = EDragAssetType::Mesh;

	if (ListingType == "3d-model")
	{
		AssetType     = EDragAssetType::Mesh;
		ImportedAsset = GetImportedObjectOfType<UStaticMesh>();
	}
	if (ListingType == "material")
	{
		AssetType     = EDragAssetType::Material;
		ImportedAsset = GetImportedObjectOfType<UMaterialInterface>();
	}
	else if (ListingType == "decal")
	{
		AssetType     = EDragAssetType::Decal;
		ImportedAsset = GetImportedObjectOfType<UMaterialInterface>();
	}

	if (ImportedAsset)
	{
		DragOperation->UpdateDraggedAsset(ImportedAsset, AssetType);
		FQuixelImportWorkflow::CompleteWorkflow();
	}
	else
	{
		FAB_LOG_ERROR("Drag and Drop failed for Megascan Asset %s", *AssetName);
		CancelWorkflow();
	}
}

void FQuixelDragDropWorkflow::CancelWorkflow()
{
	if (DragOperation)
	{
		DragOperation->CancelOperation();
	}
	FQuixelImportWorkflow::CancelWorkflow();
}

bool FQuixelDragDropWorkflow::CheckForCachedAsset(const FString& SearchPath, FAssetData& CachedMeshData) const
{
	FARFilter Filter;
	Filter.PackagePaths.Add(FName(*SearchPath));
	if (ListingType == "3d-model")
	{
		Filter.ClassPaths.Add(UStaticMesh::StaticClass()->GetClassPathName());
	}
	else if (ListingType == "material" || ListingType == "decal")
	{
		Filter.ClassPaths.Add(UMaterialInstanceConstant::StaticClass()->GetClassPathName());
	}
	Filter.bRecursivePaths = true;

	TArray<FAssetData> AssetDataList;
	IAssetRegistry::Get()->GetAssets(Filter, AssetDataList);
	if (!AssetDataList.IsEmpty())
	{
		CachedMeshData = MoveTemp(AssetDataList[0]);
		return true;
	}
	return false;
}
