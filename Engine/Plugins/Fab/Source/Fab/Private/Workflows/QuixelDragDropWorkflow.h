// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "QuixelImportWorkflow.h"

class FDragImportOperation;

class FQuixelDragDropWorkflow : public FQuixelImportWorkflow
{
public:
	FQuixelDragDropWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InListingType);

	virtual void Execute() override;

	virtual void OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;

	virtual void CompleteWorkflow() override;
	virtual void CancelWorkflow() override;

private:
	bool CheckForCachedAsset(const FString& SearchPath, FAssetData& CachedMeshData) const;
	
	FString ListingType;

	FDelegateHandle SignedUrlHandle;

	TUniquePtr<FDragImportOperation> DragOperation;
};
