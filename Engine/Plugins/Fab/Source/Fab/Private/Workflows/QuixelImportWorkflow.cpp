// Copyright Epic Games, Inc. All Rights Reserved.

#include "QuixelImportWorkflow.h"

#include "AssetToolsModule.h"
#include "FabDownloader.h"
#include "FabLog.h"
#include "FileHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Engine/StaticMesh.h"

#include "Importers/QuixelGLTFImporter.h"

#include "HAL/PlatformFileManager.h"

#include "Materials/Material.h"
#include "Pipelines/InterchangeMegascansPipeline.h"
#include "Serialization/ArchiveReplaceObjectRef.h"

#include "Utilities/AssetUtils.h"
#include "Utilities/FabAssetsCache.h"
#include "Utilities/FabLocalAssets.h"
#include "Utilities/QuixelAssetTypes.h"

// Version of the parent materials shipped with this plugin build.
// Bump the major version for any breaking change to the material hierarchy.
static const FString CurrentMaterialVersion = TEXT("1.0.0");
static const FString FabPluginMount = TEXT("/Fab/");
static const FString GameDestRoot   = TEXT("/Game");

// Returns the major version number from a "MAJOR.MINOR.PATCH" string.
static int32 GetMajorVersion(const FString& Version)
{
	int32 DotIndex;
	if (Version.FindChar(TEXT('.'), DotIndex))
	{
		return FCString::Atoi(*Version.Left(DotIndex));
	}
	return FCString::Atoi(*Version);
}

FString ExtractTierNameFromFilename(const FString& FileName)
{
	if (FileName.IsEmpty())
		return "";

	// Get clean filename without extension
	const FString CleanFileName = FPaths::GetBaseFilename(FileName);

	// Split filename by '_'
	TArray<FString> SplitString;
	CleanFileName.ParseIntoArray(SplitString, TEXT("_"), true);

	// Pick last part (tier)
	const FString TierString = SplitString.Last();

	// Convert the extracted string to an integer
	const int32 Tier = TierString.IsNumeric() ? FCString::Atoi(*TierString) : -1;
	if (Tier == 0)
		return "Raw";
	if (Tier == 1)
		return "High";
	if (Tier == 2)
		return "Medium";
	if (Tier == 3)
		return "Low";

	return "";
}

FQuixelImportWorkflow::FQuixelImportWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InDownloadURL)
	: IFabWorkflow(InAssetId, InAssetName, InDownloadURL)
{}

void FQuixelImportWorkflow::Execute()
{
	DownloadContent();
}

void FQuixelImportWorkflow::DownloadContent()
{
	const FString DownloadLocation = FFabAssetsCache::GetCacheLocation() / AssetId;

	DownloadRequest = MakeShared<FFabDownloadRequest>(AssetId, DownloadUrl, DownloadLocation, EFabDownloadType::HTTP);
	DownloadRequest->OnDownloadProgress().AddSP(this, &FQuixelImportWorkflow::OnContentDownloadProgress);
	DownloadRequest->OnDownloadComplete().AddSP(this, &FQuixelImportWorkflow::OnContentDownloadComplete);
	DownloadRequest->ExecuteRequest();

	CreateDownloadNotification();
}

void FQuixelImportWorkflow::OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& Stats)
{
	if (!Stats.bIsSuccess)
	{
		FAB_LOG_ERROR("Failed to download Megascans Asset %s", *AssetId);
		ExpireDownloadNotification(false);
		CancelWorkflow();
		return;
	}

	const FString& ZipArchive     = Stats.DownloadedFiles[0];
	const FString ExtractLocation = FPaths::GetBaseFilename(ZipArchive, false) + "_extracted";
	if (!FAssetUtils::Unzip(ZipArchive, ExtractLocation))
	{
		FAB_LOG_ERROR("Failed to unzip Megascans Asset %s", *AssetId);
		ExpireDownloadNotification(false);
		CancelWorkflow();
		return;
	}

	IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();

	TArray<FString> ImportFiles;
	FileManager.FindFiles(ImportFiles, *ExtractLocation, TEXT(".gltf"));
	FileManager.FindFiles(ImportFiles, *ExtractLocation, TEXT(".json"));

	if (ImportFiles.Num() != 2)
	{
		FAB_LOG_ERROR("Import files not found for %s", *AssetId);
		ExpireDownloadNotification(false);
		CancelWorkflow();
		return;
	}

	ExpireDownloadNotification(true);
	ImportContent(ImportFiles);
}

void FQuixelImportWorkflow::CompleteWorkflow()
{
	FAssetUtils::SyncContentBrowserToFolder(ImportLocation, !bIsDragDropWorkflow);
	IFabWorkflow::CompleteWorkflow();
}

void FQuixelImportWorkflow::OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats)
{
	SetDownloadNotificationProgress(DownloadStats.PercentComplete);
}

// Convert "/Fab/Materials/..." -> "/Game/Fab/Materials/..."
static FString MakeDestPackagePath(const FString& SourcePackagePath, const FString& PluginMount, const FString& DestRoot)
{
	// SourcePackagePath is "/Fab/Materials"
	FString Relative = SourcePackagePath;
	Relative.RemoveFromStart(PluginMount); // becomes "Materials"
	// Keep the plugin name as a folder to avoid collisions
	const FString PluginFolderName = PluginMount.Mid(1, PluginMount.Len() - 2); // "/Fab/" -> "Fab"
	return DestRoot / PluginFolderName / Relative; // "/Game/Fab/Materials"
}

static void GatherDependencies(const IAssetRegistry& AssetRegistry, const FName& PackageName, TSet<FName>& OutPackages)
{
	TArray<FName> Deps;
	FAB_LOG("GatherDependencies for %s", *PackageName.ToString());
	AssetRegistry.GetDependencies(
		PackageName,
		Deps,
		UE::AssetRegistry::EDependencyCategory::Package,
		UE::AssetRegistry::EDependencyQuery::NoRequirements
	);
	for (const FName& Dep : Deps)
	{
		const FString DepStr = Dep.ToString();
		if (DepStr.StartsWith(FabPluginMount))
		{
			bool bAlreadyInSet = false;
			OutPackages.Add(Dep, &bAlreadyInSet);
			if (!bAlreadyInSet)
			{
				FAB_LOG("Dependency: %s", *DepStr);
				GatherDependencies(AssetRegistry, Dep, OutPackages);
			}
		}
	}
}

// Gathers the source->dest package remap for one material and all its /Fab/ dependencies.
// Returns the pre-computed destination object path (e.g. "/Game/Fab/Materials/Standard/M_MS_Base.M_MS_Base").
static FString CollectMaterialPackages(
	const IAssetRegistry& AssetRegistry,
	const TSoftObjectPtr<UMaterialInterface>& Material,
	TMap<FString, FString>& InOutPackageRemap)
{
	const FSoftObjectPath SourceObjectPath = Material.ToSoftObjectPath();
	const FName SourcePackageFName = SourceObjectPath.GetLongPackageFName();
	const FString SourcePackageName = SourcePackageFName.ToString();
	const FString SourceAssetName = SourceObjectPath.GetAssetName();

	FAB_LOG("CollectMaterialPackages: %s", *SourcePackageName);

	TSet<FName> DependencyPackages;
	GatherDependencies(AssetRegistry, SourcePackageFName, DependencyPackages);

	auto AddToRemap = [&](const FString& SrcPkg)
		{
			if (InOutPackageRemap.Contains(SrcPkg))
			{
				return;  
			}

			const FString SrcFolder  = FPackageName::GetLongPackagePath(SrcPkg);
			const FString DestFolder = MakeDestPackagePath(SrcFolder, FabPluginMount, GameDestRoot);
			InOutPackageRemap.Add(SrcPkg, DestFolder / FPackageName::GetShortName(SrcPkg));
		};

	for (const FName& Dep : DependencyPackages)
	{
		AddToRemap(Dep.ToString());
	}

	AddToRemap(SourcePackageName);

	const FString SrcFolder  = FPackageName::GetLongPackagePath(SourcePackageName);
	const FString DestFolder = MakeDestPackagePath(SrcFolder, FabPluginMount, GameDestRoot);
	return DestFolder / FPackageName::GetShortName(SourcePackageName) + TEXT(".") + SourceAssetName;
}

void FQuixelImportWorkflow::CopyMaterialsToProject()
{
	if (TObjectPtr<UMegascansMaterialParentSettings> Settings = GetMutableDefault<UMegascansMaterialParentSettings>())
	{
		FString ProjectMaterialsPath = FPaths::ProjectContentDir() / TEXT("Fab/Materials");

		if (IFileManager::Get().DirectoryExists(*ProjectMaterialsPath))
		{
			const FString& StoredVersion = Settings->CopiedMaterialVersion;
			if (!StoredVersion.IsEmpty() && GetMajorVersion(StoredVersion) != GetMajorVersion(CurrentMaterialVersion))
			{
				FAB_LOG("Fab material version mismatch: project materials are version %s, plugin ships version %s. This is a breaking change.", *StoredVersion, *CurrentMaterialVersion);
			}
			else
			{
				FAB_LOG("%s already exists", *ProjectMaterialsPath);
			}
			return;
		}

		FAB_LOG("Copying Fab parent materials to %s", *ProjectMaterialsPath);

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

		// Collect all packages and precompute dest paths
		TMap<FString, FString> AllPackageRemap;

		struct FMaterialDestPaths { FString Standard; FString VT; };
		TMap<EMegascanMaterialType, FMaterialDestPaths> DestPaths;

		for (const TPair<EMegascanMaterialType, FMegascanMaterialPair>& Parent : Settings->MaterialParents)
		{
		    FMaterialDestPaths& Paths = DestPaths.Add(Parent.Key);
		    if (!Parent.Value.StandardMaterial.IsNull())
		    {
		        Paths.Standard = CollectMaterialPackages(AssetRegistry, Parent.Value.StandardMaterial, AllPackageRemap);
		    }

		    if (!Parent.Value.VTMaterial.IsNull())
		    {
		        Paths.VT = CollectMaterialPackages(AssetRegistry, Parent.Value.VTMaterial, AllPackageRemap);
		    }
		}

		if (AllPackageRemap.IsEmpty())
		{
		    FAB_LOG_ERROR("No material packages found to copy.");
		    return;
		}

		FAssetToolsModule& AssetToolsModule = FAssetToolsModule::GetModule();
		IAssetTools& AssetTools = AssetToolsModule.Get();

		const bool bOk = AssetTools.AdvancedCopyPackages(
		    AllPackageRemap,
		    /*bForceAutosave=*/true,
		    /*bCopyOverAllDestinationOverlaps=*/true
		);

		if (!bOk)
		{
		    FAB_LOG_ERROR("AdvancedCopyPackages failed, parent materials not copied.");
		    return;
		}

		 // Update MaterialParents with the copied dest paths.
		 bool bModified = false;
		 for (TPair<EMegascanMaterialType, FMegascanMaterialPair>& Parent : Settings->MaterialParents)
		 {
		    if (const FMaterialDestPaths* Paths = DestPaths.Find(Parent.Key))
		    {
		        if (!Paths->Standard.IsEmpty())
		        {
					Parent.Value.StandardMaterial = Paths->Standard;
					bModified = true;
				}

		        if (!Paths->VT.IsEmpty())
				{
					Parent.Value.VTMaterial = Paths->VT;
					bModified = true;
				}
			}
		}

		if (bModified)
		{
			Settings->CopiedMaterialVersion = CurrentMaterialVersion;
			Settings->SaveConfig();
		}
	}
}


void FQuixelImportWorkflow::ImportContent(const TArray<FString>& SourceFiles)
{
	CopyMaterialsToProject();

	const FString SourceFile = SourceFiles[0];
	const FString MetaFile   = SourceFiles[1];

	auto [MegascanId, SubType] = FQuixelAssetTypes::ExtractMeta(MetaFile, SourceFile);
	const FString TierString   = ExtractTierNameFromFilename(SourceFile);

	ImportLocation = "/Game/Fab/Megascans" / SubType / AssetName + '_' + MegascanId / TierString;
	FAssetUtils::SanitizePath(ImportLocation);

	CreateImportNotification();

	auto OnDone = [this, MegascanId](const TArray<UObject*>& Objects)
	{
		if (!Objects.IsEmpty())
		{
			ImportedObjects = Objects;
			ExpireImportNotification(true);
			UFabLocalAssets::AddLocalAsset(FPaths::GetPath(ImportLocation), AssetId);
			CompleteWorkflow();
		}
		else
		{
			FAB_LOG_ERROR("Failed to import Megascan asset: %s [%s]", *MegascanId, *AssetId);
			ExpireImportNotification(false);
			CancelWorkflow();
		}
	};

	if (SubType == "3D")
	{
		FQuixelGltfImporter::ImportGltf3DAsset(SourceFile, ImportLocation, OnDone);
	}
	else if (SubType == "Plants")
	{
		FQuixelGltfImporter::ImportGltfPlantAsset(SourceFile, ImportLocation, TierString == "Raw", OnDone);
	}
	else if (SubType == "Decals")
	{
		FQuixelGltfImporter::ImportGltfDecalAsset(SourceFile, ImportLocation, OnDone);
	}
	else if (SubType == "Imperfections")
	{
		FQuixelGltfImporter::ImportGltfImperfectionAsset(SourceFile, ImportLocation, OnDone);
	}
	else if (SubType == "Surfaces")
	{
		FQuixelGltfImporter::ImportGltfSurfaceAsset(SourceFile, ImportLocation, OnDone);
	}
	else
	{
		FAB_LOG_ERROR("Invalid Quixel asset type: %s", *SubType);
	}
}
