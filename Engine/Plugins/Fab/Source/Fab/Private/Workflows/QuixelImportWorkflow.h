// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "NotificationProgressWidget.h"

#include "AssetRegistry/AssetData.h"

#include "Workflows/FabWorkflow.h"

struct FFabDownloadStats;
class FFabDownloadRequest;

class FQuixelImportWorkflow : public IFabWorkflow
{
public:
	FQuixelImportWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InDownloadURL);

	virtual void Execute() override;

protected:
	void CopyMaterialsToProject();

	virtual void ImportContent(const TArray<FString>& SourceFiles) override;
	virtual void DownloadContent() override;

	virtual void OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;
	virtual void OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) override;

	virtual void CompleteWorkflow() override;

protected:
	bool bIsDragDropWorkflow = false;

	TSharedPtr<FFabDownloadRequest> DownloadRequest;
};
