// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/Queue.h"
#include "Containers/Ticker.h"

#include "Interfaces/IHttpRequest.h"

#define UE_API FAB_API

namespace BpiLib
{
	class IBpiLib;
}

typedef TSharedPtr<class IBuildInstaller, ESPMode::ThreadSafe> IBuildInstallerPtr;

enum class FAB_API EFabDownloadType
{
	// Download asset using HTTP
	HTTP,

	// Download asset using BuildPatchServices (for Unreal Engine Marketplace Assets)
	BuildPatchRequest,
	
	// Install plugin using BuildPatchServices (for Unreal Engine Marketplace Plugins)
	BuildPatchInstallRequest
};

struct FFabDownloadStats
{
	float PercentComplete = 0.0f;

	uint64 CompletedBytes = 0;
	uint64 TotalBytes = 0;

	uint64 DownloadStartedAt = 0;
	uint64 DownloadCompletedAt = 0;

	float DownloadSpeed = 0.0f;

	bool bIsSuccess = false;
	bool bIsCached = false;
	
	TArray<FString> DownloadedFiles;
};

class FFabDownloadRequest
{
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDownloadProgress, const FFabDownloadRequest*, const FFabDownloadStats&);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDownloadComplete, const FFabDownloadRequest*, const FFabDownloadStats&);

private:
	UE_API FString GetFilenameFromURL(const FString& URL);

	UE_API void ExecuteHTTPRequest();

	static UE_API bool LoadBuildPatchServices();
	UE_API void ExecuteBuildPatchRequest();
	UE_API void ExecuteBuildPatchInstallRequest();
	UE_API void OnPackManifestDownloaded(const FString& BaseURL);
	UE_API void OnPluginManifestDownloaded(const FString& BaseURL);

	UE_API void StartDownload();

public:
	UE_API FFabDownloadRequest(const FString& AssetID, const FString& InDownloadURL, const FString& InDownloadLocation,
	                    EFabDownloadType InDownloadType = EFabDownloadType::HTTP);

	~FFabDownloadRequest() = default;

	UE_API void ExecuteRequest();
	UE_API void Cancel();

	static UE_API void ShutdownBpsModule();

	const FFabDownloadStats& GetDownloadStats() { return DownloadStats; }

	FOnDownloadProgress& OnDownloadProgress() { return OnDownloadProgressDelegate; }
	FOnDownloadComplete& OnDownloadComplete() { return OnDownloadCompleteDelegate; }

private:
	FString AssetID;

	FString DownloadURL;

	FString DownloadLocation;

	EFabDownloadType DownloadType;

	FFabDownloadStats DownloadStats;

	FOnDownloadProgress OnDownloadProgressDelegate;
	FOnDownloadComplete OnDownloadCompleteDelegate;

	FHttpRequestPtr DownloadRequest;
	IBuildInstallerPtr BpsInstaller;

	bool bPendingCancel = false;

	TArray<uint8> ManifestData;
	FTSTicker::FDelegateHandle BpsProgressTickerHandle;
	static UE_API FTSTicker::FDelegateHandle BpsTickerHandle;
	static UE_API TUniquePtr<BpiLib::IBpiLib> BuildPatchServices;

	friend class FFabDownloadQueue;
};

class FFabDownloadQueue
{
private:
	static UE_API int32 DownloadQueueLimit;
	static UE_API TSet<FFabDownloadRequest*> DownloadQueue;
	static UE_API TQueue<FFabDownloadRequest*> WaitingQueue;

public:
	static UE_API void AddDownloadToQueue(FFabDownloadRequest* DownloadRequest);
};

#undef UE_API
