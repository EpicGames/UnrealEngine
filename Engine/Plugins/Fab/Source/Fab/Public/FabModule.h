// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

#define MS_MODULE_NAME TEXT("Fab")

class IFabModule : public IModuleInterface
{
public:
	static inline IFabModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IFabModule>(MS_MODULE_NAME);
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(MS_MODULE_NAME);
	}
};
