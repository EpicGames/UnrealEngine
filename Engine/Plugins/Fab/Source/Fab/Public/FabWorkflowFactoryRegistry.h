// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Workflows/FabWorkflowFactory.h"

#define UE_API FAB_API

class IFabWorkflowFactory;

class FFabWorkflowFactoryRegistry
{
public:
	static UE_API bool RegisterFactory(const TSharedPtr<IFabWorkflowFactory>& InFactory);
	static UE_API void UnregisterFactory(const TSharedPtr<IFabWorkflowFactory>& InFactory);

	static UE_API const TSharedPtr<IFabWorkflowFactory>& GetFactory(const FString& ImportType);
	static UE_API bool IsAssetTypeRegistered(const FString& AssetType);

private:
	static UE_API TMap<FString, TSharedPtr<IFabWorkflowFactory>> Factories;
};

#undef UE_API
