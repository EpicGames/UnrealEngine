// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ActorFactories/ActorFactory.h"

#include "AssetRegistry/AssetData.h"

#include "ActorSpawner.generated.h"

#define UE_API FAB_API

UCLASS(MinimalAPI)
class UFabPlaceholderSpawner : public UActorFactory
{
	GENERATED_UCLASS_BODY()

public:
	DECLARE_DELEGATE_OneParam(FOnActorSpawn, AActor*);
	FOnActorSpawn& OnActorSpawn() { return this->OnActorSpawnDelegate; }

protected:
	FOnActorSpawn OnActorSpawnDelegate;
};

UCLASS(MinimalAPI)
class UFabStaticMeshPlaceholderSpawner : public UFabPlaceholderSpawner
{
	GENERATED_UCLASS_BODY()

	UE_API virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	UE_API virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	UE_API virtual UObject* GetAssetFromActorInstance(AActor* Instance) override;
};

UCLASS(MinimalAPI)
class UFabSkeletalMeshPlaceholderSpawner : public UFabPlaceholderSpawner
{
	GENERATED_UCLASS_BODY()

	UE_API virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	UE_API virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	UE_API virtual UObject* GetAssetFromActorInstance(AActor* Instance) override;
};

UCLASS(MinimalAPI)
class UFabDecalPlaceholderSpawner : public UFabPlaceholderSpawner
{
	GENERATED_UCLASS_BODY()

	UE_API virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	UE_API virtual void PostSpawnActor(UObject* Asset, AActor* NewActor) override;
	UE_API virtual UObject* GetAssetFromActorInstance(AActor* Instance) override;
};

#undef UE_API
