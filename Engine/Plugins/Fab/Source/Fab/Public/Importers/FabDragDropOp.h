// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetRegistry/AssetData.h"

#include "DragAndDrop/AssetDragDropOp.h"

#define UE_API FAB_API

enum class FAB_API EDragAssetType
{
	Mesh,
	Material,
	Decal
};

class FFabDragDropOp : public FAssetDragDropOp
{
public:
	DECLARE_DELEGATE(FOnDrop);
	DRAG_DROP_OPERATOR_TYPE(FFabDragDropOp, FAssetDragDropOp)

	static UE_API TSharedPtr<FFabDragDropOp> New(FAssetData Asset, EDragAssetType InDragAssetType);

	UE_API FFabDragDropOp(const EDragAssetType InDragAssetType);
	UE_API virtual ~FFabDragDropOp() override;

	UE_API virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
	FOnDrop& OnDrop() { return this->OnDropDelegate; }

	void SetCanDropHere(bool bCanDropHere)
	{
		MouseCursor = bCanDropHere ? EMouseCursor::GrabHandClosed : EMouseCursor::SlashedCircle;
	}

	UE_API virtual void Construct() override;

	UE_API void Cancel();
	UE_API void DestroyWindow();
	UE_API void DestroySpawnedActor();

	UE_API virtual void OnDragged(const FDragDropEvent& DragDropEvent) override;
	UE_API virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override;

	TWeakObjectPtr<AActor> SpawnedActor;

protected:
	FOnDrop OnDropDelegate;

private:
	EDragAssetType DragAssetType;
	FDelegateHandle EditorApplyHandle;
};

#undef UE_API
