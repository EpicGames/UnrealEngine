// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API FAB_API

class UFbxImportUI;
class UInterchangePipelineStackOverride;

class FFabGenericImporter
{
private:
	static UE_API UObject* GetImportOptions(const FString& SourceFile, UObject* const OptionsOuter);
	static UE_API void CleanImportOptions(UObject* const Options);

public:
	static UE_API void ImportAsset(const TArray<FString>& Sources, const FString& Destination, const TFunction<void(const TArray<UObject*>&)>& Callback);
};

#undef UE_API
