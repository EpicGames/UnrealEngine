// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API FAB_API

class FAssetUtils
{
public:
	static UE_API void SanitizeFolderName(FString& AssetID);
	static UE_API void SanitizePath(FString& Path);
	static UE_API bool Unzip(const FString& Path, const FString& TargetPath);
	static UE_API void ScanForAssets(const FString& FolderPath);
	static UE_API void SyncContentBrowserToFolder(const FString& FolderPath, const bool bFocusContentBrowser = true);
	static UE_API void SyncContentBrowserToFolders(const TArray<FString>& Folders, const bool bFocusContentBrowser = true);
};

#undef UE_API
