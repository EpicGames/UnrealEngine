// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API FAB_API

class FFabAssetsCache
{
private:
	static UE_API int64 CacheExpirationTimeoutInDays;

public:
	static UE_API FString GetCacheLocation();
	static UE_API TArray<FString> GetCachedAssets();
	static UE_API int64 GetCacheSize();
	static UE_API FText GetCacheSizeString();
	static UE_API void ClearCache();
	static UE_API bool IsCached(const FString& AssetId, int64 DownloadSize);
	static UE_API FString GetCachedFile(const FString AssetId);
	static UE_API FString CacheAsset(const FString& DownloadedAssetPath);
};

#undef UE_API
