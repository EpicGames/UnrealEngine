// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/Async.h"
#include "FabWorkflow.generated.h"

#define UE_API FAB_API

struct FFabDownloadStats;
class FFabDownloadRequest;
class SNotificationItem;
class SNotificationProgressWidget;

USTRUCT()
struct FFabAssetMetadata
{
	GENERATED_BODY()

	UPROPERTY()
	FString AssetId;

	UPROPERTY()
	FString AssetName;

	UPROPERTY()
	FString AssetType;

	UPROPERTY()
	FString ListingType;

	UPROPERTY()
	FString AssetNamespace;

	UPROPERTY()
	TArray<FString> DistributionPointBaseUrls;

	UPROPERTY()
	bool IsQuixel = false;
};

class IFabWorkflow : public TSharedFromThis<IFabWorkflow>
{
public:
	DECLARE_DELEGATE(FOnFabWorkflowComplete);
	DECLARE_DELEGATE(FOnFabWorkflowCancel);

public:
	IFabWorkflow(const FString& InAssetId, const FString& InAssetName, const FString& InDownloadUrl)
		: AssetId(InAssetId)
		, AssetName(InAssetName)
		, DownloadUrl(InDownloadUrl)
	{}

	virtual ~IFabWorkflow() = default;

	virtual void Execute() = 0;

	FOnFabWorkflowComplete& OnFabWorkflowComplete() { return OnFabWorkflowCompleteDelegate; }
	FOnFabWorkflowCancel& OnFabWorkflowCancel() { return OnFabWorkflowCancelDelegate; }

	virtual const TArray<UObject*>& GetImportedObjects() const { return ImportedObjects; }

	template <class T>
	T* GetImportedObjectOfType() const
	{
		if (UObject* const* FoundObject = ImportedObjects.FindByPredicate([](const auto& O) { return O->IsA(T::StaticClass()); }))
		{
			return Cast<T>(*FoundObject);
		}
		return nullptr;
	}

protected:
	virtual void ImportContent(const TArray<FString>& SourceFiles) {};
	virtual void DownloadContent() = 0;

	virtual void OnContentDownloadProgress(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) = 0;
	virtual void OnContentDownloadComplete(const FFabDownloadRequest* Request, const FFabDownloadStats& DownloadStats) = 0;

	// Deferred via AsyncTask so the current call stack unwinds before the
	// delegate fires, which may release the last shared reference to this
	// workflow. Captures a shared pointer to prevent premature destruction.
	virtual void CompleteWorkflow()
	{
		TSharedRef<IFabWorkflow> Self = AsShared();
		AsyncTask(ENamedThreads::GameThread, [Self]() { Self->OnFabWorkflowComplete().ExecuteIfBound(); });
	}
	virtual void CancelWorkflow()
	{
		TSharedRef<IFabWorkflow> Self = AsShared();
		AsyncTask(ENamedThreads::GameThread, [Self]() { Self->OnFabWorkflowCancel().ExecuteIfBound(); });
	}

	UE_API virtual void CreateDownloadNotification();
	UE_API virtual void SetDownloadNotificationProgress(const float Progress) const;
	UE_API virtual void ExpireDownloadNotification(bool bSuccess) const;

	UE_API virtual void CreateImportNotification();
	UE_API virtual void ExpireImportNotification(bool bSuccess) const;

public:
	FString AssetId;

protected:
	FString AssetName;
	FString DownloadUrl;

	FString ImportLocation;
	TArray<UObject*> ImportedObjects;

	TSharedPtr<SNotificationItem> DownloadProgressNotification;
	TSharedPtr<SNotificationItem> ImportProgressNotification;
	TSharedPtr<SNotificationProgressWidget> ProgressWidget;

private:
	FOnFabWorkflowComplete OnFabWorkflowCompleteDelegate;
	FOnFabWorkflowCancel OnFabWorkflowCancelDelegate;
};

#undef UE_API
