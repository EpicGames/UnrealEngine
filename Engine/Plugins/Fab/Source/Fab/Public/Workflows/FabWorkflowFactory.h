// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FabWorkflow.h"

class IFabWorkflowFactory
{
public:
	IFabWorkflowFactory() = default;
	virtual ~IFabWorkflowFactory() = default;

	IFabWorkflowFactory(const IFabWorkflowFactory&) = delete;
	IFabWorkflowFactory(IFabWorkflowFactory&&) = delete;

	IFabWorkflowFactory& operator=(const IFabWorkflowFactory&) = delete;
	IFabWorkflowFactory& operator=(IFabWorkflowFactory&&) = delete;

	virtual bool CanImportAssetType(const FString& InAssetType)
	{
		return GetImportAssetTypes().Contains(InAssetType);
	}

	virtual const TArray<FString>& GetImportAssetTypes() = 0;

	virtual TSharedPtr<IFabWorkflow> Create(const FFabAssetMetadata& InImportAssetMetadata, const FString& InDownloadUrl = "") = 0;
};
